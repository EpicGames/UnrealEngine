// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/CastableTypeTable.h"

namespace UE::Sequencer
{

const void* FDynamicExtensionContainer::CastDynamic(FViewModelTypeID Type) const
{
	for (const FDynamicExtensionInfo& DynamicExtension : DynamicExtensions)
	{
		const void* Result = DynamicExtension.Extension ? DynamicExtension.TypeTable->Cast(DynamicExtension.Extension.Get(), Type.GetTypeID()) : nullptr;
		if (Result)
		{
			return Result;
		}
	}
	return nullptr;
}

void FDynamicExtensionContainer::RemoveDynamicExtension(FViewModelTypeID Type)
{
	for (int32 Index = DynamicExtensions.Num()-1; Index >= 0; --Index)
	{
		const FDynamicExtensionInfo& DynamicExtension = DynamicExtensions[Index];
		if (DynamicExtension.Extension && DynamicExtension.TypeTable->Cast(DynamicExtension.Extension.Get(), Type.GetTypeID()) != nullptr)
		{
			// null it out so we can add/remove while we're iterating
			DynamicExtensions[Index].Extension = nullptr;
			return;
		}
	}
}

FDynamicExtensionContainerIterator::FDynamicExtensionContainerIterator(IteratorType&& InIterator, FViewModelTypeID InType)
	: CurrentExtension(nullptr)
	, Iterator(MoveTemp(InIterator))
	, Type(InType)
{
	for ( ; Iterator; ++Iterator)
	{
		CurrentExtension = Iterator->Extension ? const_cast<void*>(Iterator->TypeTable->Cast(Iterator->Extension.Get(), Type.GetTypeID())) : nullptr;
		if (CurrentExtension)
		{
			break;
		}
	}
}

FDynamicExtensionContainerIterator& FDynamicExtensionContainerIterator::operator++()
{
	++Iterator;

	for (CurrentExtension = nullptr; Iterator; ++Iterator)
	{
		CurrentExtension = Iterator->Extension ? const_cast<void*>(Iterator->TypeTable->Cast(Iterator->Extension.Get(), Type.GetTypeID())) : nullptr;
		if (CurrentExtension)
		{
			break;
		}
	}

	return *this;
}

bool operator!=(const FDynamicExtensionContainerIterator& A, const FDynamicExtensionContainerIterator& B)
{
	return A.Iterator != B.Iterator || A.Type != B.Type;
}

} // namespace UE::Sequencer

