// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/HierarchicalCacheExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE::Sequencer
{


FHierarchicalCacheExtension::~FHierarchicalCacheExtension()
{
}

void FHierarchicalCacheExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	WeakOwnerModel = InWeakOwner;
}

void FHierarchicalCacheExtension::Initialize(const FViewModelPtr& InRootModel)
{
	WeakRootModel = InRootModel;
}

void FHierarchicalCacheExtension::OnHierarchyUpdated()
{
	UpdateCachedFlags();
}

void FHierarchicalCacheExtension::UpdateCachedFlags()
{
	FViewModelPtr Owner = WeakOwnerModel.Pin();
	if (!Owner)
	{
		return;
	}

	// Trigger our event, allowing caches to be added if necessary
	PreUpdateCachesEvent.Broadcast(Owner);

	TArray<IHierarchicalCache*> HierarchicalCaches;

	for (IHierarchicalCache& HierarchicalCache : Owner->FilterDynamic<IHierarchicalCache>())
	{
		HierarchicalCaches.Add(&HierarchicalCache);
	}

	if (FViewModelPtr RootModel = WeakRootModel.Pin())
	{
		UpdateAllCachedFlags(RootModel, HierarchicalCaches);
	}
}

void FHierarchicalCacheExtension::UpdateAllCachedFlags(const FViewModelPtr& ViewModel, TArrayView<IHierarchicalCache* const> HierarchicalCaches)
{
	for (IHierarchicalCache* HierarchicalCache : HierarchicalCaches)
	{
		HierarchicalCache->BeginUpdate();
	}

	if (ViewModel)
	{
		UpdateCachedFlagsForModel(ViewModel, HierarchicalCaches);
	}

	for (IHierarchicalCache* HierarchicalCache : HierarchicalCaches)
	{
		HierarchicalCache->EndUpdate();
	}
}

void FHierarchicalCacheExtension::UpdateCachedFlagsForModel(const FViewModelPtr& ViewModel, TArrayView<IHierarchicalCache* const> HierarchicalCaches)
{
	// Visit parent-first caches, and call pre-visit children
	for (IHierarchicalCache* HierarchicalCache : HierarchicalCaches)
	{
		HierarchicalCache->PreVisitChildren(ViewModel);
	}

	for (const FViewModelPtr& Child : ViewModel->GetChildren(ModelListFilter))
	{
		UpdateCachedFlagsForModel(Child, HierarchicalCaches);
	}

	// Call post-visit children and visit child-first caches
	for (IHierarchicalCache* HierarchicalCache : HierarchicalCaches)
	{
		HierarchicalCache->PostVisitChildren(ViewModel);
	}
}

} // namespace UE::Sequencer

