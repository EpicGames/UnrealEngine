// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IDeactivatableExtension.h"

namespace UE::Sequencer
{

ECachedDeactiveState CombinePropagatedChildFlags(const ECachedDeactiveState ParentFlags, ECachedDeactiveState CombinedChildFlags)
{
	// If this item is deactivatable, but not deactivated, do not inherit deactivated state
	if ((ParentFlags & (ECachedDeactiveState::Deactivatable | ECachedDeactiveState::Deactivated)) == ECachedDeactiveState::Deactivatable
		&& EnumHasAnyFlags(CombinedChildFlags, ECachedDeactiveState::Deactivated))
	{
		CombinedChildFlags &= ~ECachedDeactiveState::Deactivated;
		CombinedChildFlags |= ECachedDeactiveState::PartiallyDeactivatedChildren;
	}

	return ParentFlags | CombinedChildFlags;
}

ECachedDeactiveState FDeactiveStateCacheExtension::ComputeFlagsForModel(const FViewModelPtr& ViewModel)
{
	ECachedDeactiveState ParentFlags = IndividualItemFlags.Last();
	ECachedDeactiveState ThisModelFlags = ECachedDeactiveState::None;

	if (EnumHasAnyFlags(ParentFlags, ECachedDeactiveState::Deactivated | ECachedDeactiveState::ImplicitlyDeactivatedByParent))
	{
		ThisModelFlags |= ECachedDeactiveState::ImplicitlyDeactivatedByParent;
	}

	if (TViewModelPtr<IDeactivatableExtension> Deactivatable = ViewModel.ImplicitCast())
	{
		ThisModelFlags |= ECachedDeactiveState::Deactivatable;
		if (Deactivatable->IsDeactivated())
		{
			ThisModelFlags |= ECachedDeactiveState::Deactivated;
		}
		if (Deactivatable->IsInheritable())
		{
			ThisModelFlags |= ECachedDeactiveState::Inheritable;
		}
	}

	return ThisModelFlags;
}

void FDeactiveStateCacheExtension::PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedDeactiveState& OutThisModelFlags, ECachedDeactiveState& OutPropagateToParentFlags)
{
	const bool bIsInheritable = EnumHasAnyFlags(OutThisModelFlags, ECachedDeactiveState::Inheritable);
	if (!bIsInheritable)
	{
		return;
	}

	// --------------------------------------------------------------------
	// Handle deactivated state propagation
	const bool bIsDeactivatable              = EnumHasAnyFlags(OutThisModelFlags, ECachedDeactiveState::Deactivatable);
	const bool bIsDeactivated                = EnumHasAnyFlags(OutThisModelFlags, ECachedDeactiveState::Deactivated);
	const bool bHasDeactivatableChildren     = EnumHasAnyFlags(OutThisModelFlags, ECachedDeactiveState::DeactivatableChildren);
	const bool bSiblingsPartiallyDeactivated = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedDeactiveState::PartiallyDeactivatedChildren);
	const bool bSiblingsFullyDeactivated     = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedDeactiveState::Deactivated);
	const bool bHasAnyDeactivatableSiblings  = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedDeactiveState::DeactivatableChildren);

	if (bIsDeactivated)
	{
		if (!bSiblingsPartiallyDeactivated)
		{
			if (bHasAnyDeactivatableSiblings && !bSiblingsFullyDeactivated)
			{
				// This is the first deactivated deactivatable within the parent, but we know there are already other deactivatables so this has to be partially deactivated
				OutPropagateToParentFlags |= ECachedDeactiveState::PartiallyDeactivatedChildren;
			}
			else
			{
				// Parent is (currently) fully deactivated because it contains no other deactivatable children, and is not already partially deactivated
				OutPropagateToParentFlags |= ECachedDeactiveState::Deactivated;
			}
		}
	}
	else if (bIsDeactivatable && bSiblingsFullyDeactivated)
	{
		// Parent is no longer fully deactivated because it contains an activated deactivatable
		OutPropagateToParentFlags |= ECachedDeactiveState::PartiallyDeactivatedChildren;
		OutPropagateToParentFlags &= ~ECachedDeactiveState::Deactivated;
	}

	if (bIsDeactivatable)
	{
		OutPropagateToParentFlags |= ECachedDeactiveState::DeactivatableChildren;
	}
}

} // namespace UE::Sequencer
