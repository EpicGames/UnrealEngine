// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IGeometryExtension.h"

#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE
{
namespace Sequencer
{

float IGeometryExtension::UpdateVirtualGeometry(float InitialVirtualPosition, TSharedPtr<FViewModel> InItem)
{
	float VirtualOffset = InitialVirtualPosition;

	if (IOutlinerExtension* OutlinerExtension = InItem->CastThis<IOutlinerExtension>())
	{
		FOutlinerSizing Sizing = OutlinerExtension->GetOutlinerSizing();
		VirtualOffset += Sizing.GetTotalHeight();

		// Accumulate children
		for (TSharedPtr<FViewModel> Child : InItem->GetChildren())
		{
			VirtualOffset = UpdateVirtualGeometry(VirtualOffset, Child);
		}

		FVirtualGeometry Geometry(InitialVirtualPosition, Sizing.GetTotalHeight(), VirtualOffset);

		if (IGeometryExtension* VirtualGeometryExtension = InItem->CastThis<IGeometryExtension>())
		{
			VirtualGeometryExtension->ReportVirtualGeometry(Geometry);
		}

		// Report geometry for any track area list models
		if (ITrackAreaExtension* TrackAreaExtension = InItem->CastThis<ITrackAreaExtension>())
		{
			for (TTypedIterator<IGeometryExtension, FViewModelVariantIterator>
				 It(TrackAreaExtension->GetTrackAreaModelList()); It; ++It)
			{
				It->ReportVirtualGeometry(Geometry);
			}

			for (TTypedIterator<IGeometryExtension, FViewModelVariantIterator>
				It(TrackAreaExtension->GetTopLevelChildTrackAreaModels()); It; ++It)
			{
				It->ReportVirtualGeometry(Geometry);
			}
		}
	}
	else
	{
		for (TSharedPtr<FViewModel> Child : InItem->GetChildren())
		{
			VirtualOffset = UpdateVirtualGeometry(VirtualOffset, Child);
		}
	}

	return VirtualOffset;
}

} // namespace Sequencer
} // namespace UE

