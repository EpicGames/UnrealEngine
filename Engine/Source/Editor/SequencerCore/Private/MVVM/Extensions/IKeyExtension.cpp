// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IKeyExtension.h"

namespace UE::Sequencer
{

bool FCachedKeys::FCachedKey::IsValid() const
{
	return Handle != FKeyHandle::Invalid();
}

void FCachedKeys::FCachedKey::Reset()
{
	Handle = FKeyHandle::Invalid();
}

void FCachedKeys::GetKeysInRange(const TRange<FFrameTime>& Range, TArrayView<const FFrameTime>* OutKeyTimes, TArrayView<const FKeyHandle>* OutHandles) const
{
	GetKeysInRangeWithBounds(Range, OutKeyTimes, OutHandles, nullptr, nullptr);
}

void FCachedKeys::GetKeysInRangeWithBounds(const TRange<FFrameTime>& Range, TArrayView<const FFrameTime>* OutKeyTimes, TArrayView<const FKeyHandle>* OutHandles, FCachedKey* OutLeadingKey, FCachedKey* OutTrailingKey) const
{
	// Binary search the first time that's >= the lower bound, minus the dilation amount
	const int32 FirstVisibleIndex = Algo::LowerBound(KeyTimes, Range.GetLowerBoundValue());
	// Binary search the last time that's > the upper bound
	const int32 LastVisibleIndex = Algo::UpperBound(KeyTimes, Range.GetUpperBoundValue());

	int32 Num = LastVisibleIndex - FirstVisibleIndex;
	if (KeyTimes.IsValidIndex(FirstVisibleIndex) && LastVisibleIndex <= KeyTimes.Num())
	{
		if (OutKeyTimes)
		{
			*OutKeyTimes = MakeArrayView(&KeyTimes[FirstVisibleIndex], Num);
		}

		if (OutHandles)
		{
			*OutHandles = MakeArrayView(&KeyHandles[FirstVisibleIndex], Num);
		}
	}
	else
	{
		if (OutKeyTimes)
		{
			*OutKeyTimes = TArrayView<const FFrameTime>();
		}

		if (OutHandles)
		{
			*OutHandles = TArrayView<const FKeyHandle>();
		}
	}

	if (OutLeadingKey)
	{
		if (KeyTimes.IsValidIndex(FirstVisibleIndex-1))
		{
			*OutLeadingKey = FCachedKey{ KeyTimes[FirstVisibleIndex-1], KeyHandles[FirstVisibleIndex-1] };
		}
		else
		{
			*OutLeadingKey = FCachedKey();
		}
	}

	if (OutTrailingKey)
	{
		if (KeyTimes.IsValidIndex(LastVisibleIndex))
		{
			*OutTrailingKey = FCachedKey{ KeyTimes[LastVisibleIndex], KeyHandles[LastVisibleIndex] };
		}
		else
		{
			*OutTrailingKey = FCachedKey();
		}
	}
}

} // namespace UE::Sequencer

