// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/ILockableExtension.h"

namespace UE::Sequencer
{

ECachedLockState FLockStateCacheExtension::ComputeFlagsForModel(const FViewModelPtr& ViewModel)
{
	ECachedLockState& ParentFlags = IndividualItemFlags.Last();
	ECachedLockState ThisModelFlags = ECachedLockState::None;

	if (EnumHasAnyFlags(ParentFlags, ECachedLockState::Locked | ECachedLockState::ImplicitlyLockedByParent))
	{
		ThisModelFlags |= ECachedLockState::ImplicitlyLockedByParent;
	}

	if (TViewModelPtr<ILockableExtension> Lockable = ViewModel.ImplicitCast())
	{
		ThisModelFlags |= ECachedLockState::Lockable;

		switch (Lockable->GetLockState())
		{
		case ELockableLockState::Locked:
			ThisModelFlags |= ECachedLockState::Locked;
			break;
		case ELockableLockState::PartiallyLocked:
			// Lockable, but not locked - any parent must not be fully locked
			if (EnumHasAnyFlags(ParentFlags, ECachedLockState::Locked))
			{
				ParentFlags |= ECachedLockState::PartiallyLockedChildren;
				ParentFlags &= ~ECachedLockState::Locked;
			}
			break;
		}
	}

	return ThisModelFlags;
}

void FLockStateCacheExtension::PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedLockState& OutThisModelFlags, ECachedLockState& OutPropagateToParentFlags)
{
	// --------------------------------------------------------------------
	// Handle lock state propagation
	const bool bIsLockable              = EnumHasAnyFlags(OutThisModelFlags, ECachedLockState::Lockable);
	const bool bIsLocked                = EnumHasAnyFlags(OutThisModelFlags, ECachedLockState::Locked);
	const bool bHasLockableChildren     = EnumHasAnyFlags(OutThisModelFlags, ECachedLockState::LockableChildren);
	const bool bSiblingsPartiallyLocked = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedLockState::PartiallyLockedChildren);
	const bool bSiblingsFullyLocked     = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedLockState::Locked);
	const bool bHasAnyLockableSiblings  = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedLockState::LockableChildren);

	if (bIsLocked)
	{
		if (!bSiblingsPartiallyLocked)
		{
			if (bHasAnyLockableSiblings && !bSiblingsFullyLocked)
			{
				// This is the first locked lockable within the parent, but we know there are already other lockables so this has to be partially locked
				OutPropagateToParentFlags |= ECachedLockState::PartiallyLockedChildren;
			}
			else
			{
				// Parent is (currently) fully locked because it contains no other lockable children, and is not already partially locked
				OutPropagateToParentFlags |= ECachedLockState::Locked;
			}
		}
	}
	else if (bIsLockable && bSiblingsFullyLocked)
	{
		// Parent is no longer fully locked because it contains an unLocked lockable
		OutPropagateToParentFlags |= ECachedLockState::PartiallyLockedChildren;
		OutPropagateToParentFlags &= ~ECachedLockState::Locked;
	}

	if (bIsLockable)
	{
		OutPropagateToParentFlags |= ECachedLockState::LockableChildren;
	}
}

} // namespace UE::Sequencer

