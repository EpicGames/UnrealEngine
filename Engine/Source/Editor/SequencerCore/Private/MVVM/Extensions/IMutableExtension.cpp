// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IMutableExtension.h"

namespace UE::Sequencer
{
ECachedMuteState CombinePropagatedChildFlags(ECachedMuteState ParentFlags, ECachedMuteState CombinedChildFlags)
{
	// If this item is mutable, but not muted, do not inherit mute state
	if ((ParentFlags & (ECachedMuteState::Mutable | ECachedMuteState::Muted)) == ECachedMuteState::Mutable && EnumHasAnyFlags(CombinedChildFlags, ECachedMuteState::Muted))
	{
		CombinedChildFlags &= ~ECachedMuteState::Muted;
		CombinedChildFlags |= ECachedMuteState::PartiallyMutedChildren;
	}

	return ParentFlags | CombinedChildFlags;
}

ECachedMuteState FMuteStateCacheExtension::ComputeFlagsForModel(const FViewModelPtr& ViewModel)
{
	ECachedMuteState ParentFlags = IndividualItemFlags.Last();
	ECachedMuteState ThisModelFlags = ECachedMuteState::None;

	if (EnumHasAnyFlags(ParentFlags, ECachedMuteState::Muted | ECachedMuteState::ImplicitlyMutedByParent))
	{
		ThisModelFlags |= ECachedMuteState::ImplicitlyMutedByParent;
	}

	if (TViewModelPtr<IMutableExtension> Mutable = ViewModel.ImplicitCast())
	{
		ThisModelFlags |= ECachedMuteState::Mutable;
		if (Mutable->IsMuted())
		{
			ThisModelFlags |= ECachedMuteState::Muted;
		}
		if (Mutable->IsInheritable())
		{
			ThisModelFlags |= ECachedMuteState::Inheritable;
		}
	}

	return ThisModelFlags;
}

void FMuteStateCacheExtension::PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedMuteState& OutThisModelFlags, ECachedMuteState& OutPropagateToParentFlags)
{
	const bool bIsInheritable          = EnumHasAnyFlags(OutThisModelFlags, ECachedMuteState::Inheritable);
	if (!bIsInheritable)
	{
		return;
	}

	// --------------------------------------------------------------------
	// Handle mute state propagation
	const bool bIsMutable              = EnumHasAnyFlags(OutThisModelFlags, ECachedMuteState::Mutable);
	const bool bIsMuted                = EnumHasAnyFlags(OutThisModelFlags, ECachedMuteState::Muted);
	const bool bHasMutableChildren     = EnumHasAnyFlags(OutThisModelFlags, ECachedMuteState::MutableChildren);
	const bool bSiblingsPartiallyMuted = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedMuteState::PartiallyMutedChildren);
	const bool bSiblingsFullyMuted     = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedMuteState::Muted);
	const bool bHasAnyMutableSiblings  = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedMuteState::MutableChildren);

	if (bIsMuted)
	{
		if (!bSiblingsPartiallyMuted)
		{
			if (bHasAnyMutableSiblings && !bSiblingsFullyMuted)
			{
				// This is the first muted mutable within the parent, but we know there are already other mutables so this has to be partially muted
				OutPropagateToParentFlags |= ECachedMuteState::PartiallyMutedChildren;
			}
			else
			{
				// Parent is (currently) fully muted because it contains no other mutable children, and is not already partially muted
				OutPropagateToParentFlags |= ECachedMuteState::Muted;
			}
		}
	}
	else if (bIsMutable && bSiblingsFullyMuted)
	{
		// Parent is no longer fully Muted because it contains an unmuted Mutable
		OutPropagateToParentFlags |= ECachedMuteState::PartiallyMutedChildren;
		OutPropagateToParentFlags &= ~ECachedMuteState::Muted;
	}

	if (bIsMutable)
	{
		OutPropagateToParentFlags |= ECachedMuteState::MutableChildren;
	}
}

} // namespace UE::Sequencer

