// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IOutlinerExtension.h"

#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/StringBuilder.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/SNullWidget.h"

namespace UE::Sequencer
{

void GetParentPathImpl(TParentModelIterator<IOutlinerExtension>& Iterator, FStringBuilderBase& OutString)
{
	// Store the current tree item
	TViewModelPtr<IOutlinerExtension> TreeItem = *Iterator;
	++Iterator;

	if (Iterator)
	{
		GetParentPathImpl(Iterator, OutString);
	}

	// Add our identifier last
	TreeItem->GetIdentifier().AppendString(OutString);
	OutString.AppendChar('.');
}

void IOutlinerExtension::ToggleSelectionState(EOutlinerSelectionState InState, bool bInValue)
{
	EOutlinerSelectionState CurrentState = GetSelectionState();
	if (bInValue)
	{
		SetSelectionState(CurrentState | InState);
	}
	else
	{
		SetSelectionState(CurrentState & (~InState));
	}
}

FText IOutlinerExtension::GetLabel() const
{
	return FText::FromName(GetIdentifier());
}

FSlateFontInfo IOutlinerExtension::GetLabelFont() const
{
	return FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont");
}

FSlateColor IOutlinerExtension::GetLabelColor() const
{
	return FSlateColor::UseForeground();
}

FText IOutlinerExtension::GetLabelToolTipText() const
{
	return FText();
}

const FSlateBrush* IOutlinerExtension::GetIconBrush() const
{
	return nullptr;
}

FSlateColor IOutlinerExtension::GetIconTint() const
{
	return GetLabelColor();
}

const FSlateBrush* IOutlinerExtension::GetIconOverlayBrush() const
{
	return nullptr;
}

FText IOutlinerExtension::GetIconToolTipText() const
{
	return FText();
}

bool IOutlinerExtension::HasBackground() const
{
	return true;
}

FString IOutlinerExtension::GetPathName(const FViewModel& Item)
{
	TStringBuilder<256> Builder;
	IOutlinerExtension::GetPathName(Item, Builder);
	return Builder.ToString();
}

FString IOutlinerExtension::GetPathName(const TSharedPtr<const FViewModel> Item)
{
	if (const FViewModel* Model = Item.Get())
	{
		return GetPathName(*Model);
	}
	return FString();
}

FString IOutlinerExtension::GetPathName(const TSharedPtr<FViewModel> Item)
{
	if (const FViewModel* Model = Item.Get())
	{
		return GetPathName(*Model);
	}
	return FString();
}

FString IOutlinerExtension::GetPathName(const TWeakPtr<const FViewModel> Item)
{
	if (TSharedPtr<const FViewModel> Model = Item.Pin())
	{
		return GetPathName(*Model.Get());
	}
	return FString();
}

void IOutlinerExtension::GetPathName(const FViewModel& Item, FStringBuilderBase& OutString)
{
	// Add the parent path first
	TParentModelIterator<IOutlinerExtension> Ancestors = const_cast<FViewModel&>(Item).GetAncestorsOfType<IOutlinerExtension>();
	if (Ancestors)
	{
		GetParentPathImpl(Ancestors, OutString);
	}

	// Add our identifier last
	if (const IOutlinerExtension* TreeItem = Item.CastThis<IOutlinerExtension>())
	{
		TreeItem->GetIdentifier().AppendString(OutString);
	}
}

TSharedRef<SWidget> IOutlinerExtension::CreateOutlinerView(const FCreateOutlinerViewParams& InParams)
{
	return SNullWidget::NullWidget;
}

TSharedPtr<SWidget> IOutlinerExtension::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	return nullptr;
}

} // namespace UE::Sequencer

