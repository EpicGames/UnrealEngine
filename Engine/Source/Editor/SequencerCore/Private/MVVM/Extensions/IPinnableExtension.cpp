// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IPinnableExtension.h"

#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "SequencerCoreFwd.h"

namespace UE
{
namespace Sequencer
{

bool FPinnableExtensionShim::IsPinned() const
{
	return bIsPinned || bIsInPinnedBranch;
}

void FPinnableExtensionShim::SetPinned(bool bInIsPinned)
{
	bIsPinned = bInIsPinned;
}

void FPinnableExtensionShim::ReportPinnedParent(bool bInIsInPinnedBranch)
{
	bIsInPinnedBranch = bInIsInPinnedBranch;
}

void FPinnableExtensionShim::UpdateCachedPinnedState(TSharedPtr<FViewModel> RootModel)
{
	// Only go through the root-level items, and mark their children as pinned or not
	const bool bIncludeRootModel = true;
	for (const TViewModelPtr<IPinnableExtension>& Item : RootModel->GetChildrenOfType<IPinnableExtension>())
	{
		const bool bIsParentPinned = Item->IsPinned();

		// Report parent state
		for (const TViewModelPtr<IPinnableExtension>& Child : Item.AsModel()->GetDescendantsOfType<IPinnableExtension>())
		{
			Child->ReportPinnedParent(bIsParentPinned);
		}
	}
}

} // namespace Sequencer
} // namespace UE

