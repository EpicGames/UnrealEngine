// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/ViewModelPtr.h"

namespace UE
{
namespace Sequencer
{

void IRecyclableExtension::CallOnRecycle(const TViewModelPtr<IRecyclableExtension>& RecyclableItem)
{
	if (RecyclableItem)
	{
		RecyclableItem->OnRecycle();
	}
}

} // namespace Sequencer
} // namespace UE

