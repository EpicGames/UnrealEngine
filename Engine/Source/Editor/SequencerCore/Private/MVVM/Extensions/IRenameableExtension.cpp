// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IRenameableExtension.h"

#include "Containers/UnrealString.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

namespace UE
{
namespace Sequencer
{

bool IRenameableExtension::IsRenameValid(const FText& NewName, FText& OutErrorMessage) const
{
	if (NewName.IsEmpty())
	{
		OutErrorMessage = NSLOCTEXT("Sequencer", "RenameFailed_LeftBlank", "Labels cannot be left blank");
		return false;
	}
	else if (NewName.ToString().Len() >= NAME_SIZE)
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("Sequencer", "RenameFailed_TooLong", "Names must be less than {0} characters long"), NAME_SIZE);
		return false;
	}

	return IsRenameValidImpl(NewName, OutErrorMessage);
}

} // namespace Sequencer
} // namespace UE

