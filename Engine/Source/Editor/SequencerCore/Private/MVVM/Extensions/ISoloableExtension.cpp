// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/ISoloableExtension.h"

namespace UE::Sequencer
{

ECachedSoloState FSoloStateCacheExtension::ComputeFlagsForModel(const FViewModelPtr& ViewModel)
{
	ECachedSoloState ParentFlags = IndividualItemFlags.Last();
	ECachedSoloState ThisModelFlags = ECachedSoloState::None;

	if (EnumHasAnyFlags(ParentFlags, ECachedSoloState::Soloed | ECachedSoloState::ImplicitlySoloedByParent))
	{
		ThisModelFlags |= ECachedSoloState::ImplicitlySoloedByParent;
	}

	if (TViewModelPtr<ISoloableExtension> Soloable = ViewModel.ImplicitCast())
	{
		ThisModelFlags |= ECachedSoloState::Soloable;
		if (Soloable->IsSolo())
		{
			ThisModelFlags |= ECachedSoloState::Soloed;
		}
	}

	return ThisModelFlags;
}

void FSoloStateCacheExtension::PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedSoloState& OutThisModelFlags, ECachedSoloState& OutPropagateToParentFlags)
{
	// --------------------------------------------------------------------
	// Handle solo state propagation
	const bool bIsSoloable              = EnumHasAnyFlags(OutThisModelFlags, ECachedSoloState::Soloable);
	const bool bIsSoloed                = EnumHasAnyFlags(OutThisModelFlags, ECachedSoloState::Soloed);
	const bool bHasSoloableChildren     = EnumHasAnyFlags(OutThisModelFlags, ECachedSoloState::SoloableChildren);
	const bool bSiblingsPartiallySoloed = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedSoloState::PartiallySoloedChildren);
	const bool bSiblingsFullySoloed     = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedSoloState::Soloed);
	const bool bHasAnySoloableSiblings  = EnumHasAnyFlags(OutPropagateToParentFlags, ECachedSoloState::SoloableChildren);

	if (bIsSoloed)
	{
		if (!bSiblingsPartiallySoloed)
		{
			if (bHasAnySoloableSiblings && !bSiblingsFullySoloed)
			{
				// This is the first soloed soloable within the parent, but we know there are already other soloables so this has to be partially soloed
				OutPropagateToParentFlags |= ECachedSoloState::PartiallySoloedChildren;
			}
			else
			{
				// Parent is (currently) fully soloed because it contains no other soloable children, and is not already partially soloed
				OutPropagateToParentFlags |= ECachedSoloState::Soloed;
			}
		}
	}
	else if (bIsSoloable && bSiblingsFullySoloed)
	{
		// Parent is no longer fully Soloed because it contains an unsoloed soloable
		OutPropagateToParentFlags |= ECachedSoloState::PartiallySoloedChildren;
		OutPropagateToParentFlags &= ~ECachedSoloState::Soloed;
	}

	if (bIsSoloable)
	{
		OutPropagateToParentFlags |= ECachedSoloState::SoloableChildren;
	}
}

} // namespace UE::Sequencer

