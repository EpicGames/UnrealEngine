// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/ISortableExtension.h"

#include "Algo/StableSort.h"
#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/Optional.h"
#include "Templates/TypeHash.h"

namespace UE
{
namespace Sequencer
{

bool FSortingKey::ComparePriorityFirst(const FSortingKey& A, const FSortingKey& B)
{
	if (A.Priority == B.Priority)
	{
		// Compare identifiers
		const int32 Compare = A.DisplayName.CompareToCaseIgnored(B.DisplayName);
		if (Compare != 0)
		{
			return Compare < 0;
		}

		// Note in this case, it's possible that one of the items does not have a custom order (ie. -1). 
		// That's fine because -1 will be sorted to the top and generally if new items are added and 
		// there is ambiguity, we would set a max custom order.
		return A.CustomOrder < B.CustomOrder;
	}
	return A.Priority < B.Priority;
}

bool FSortingKey::CompareCustomOrderFirst(const FSortingKey& A, const FSortingKey& B)
{
	if (A.CustomOrder >= 0 && B.CustomOrder >= 0)
	{
		// Both items have custom orders, use those
		return A.CustomOrder < B.CustomOrder;
	}
	return ComparePriorityFirst(A, B);
}

void ISortableExtension::SortChildren(TSharedPtr<FViewModel> ParentModel, ESortingMode SortingMode)
{
	TOptional<FViewModelChildren> OutlinerChildren = ParentModel->FindChildList(EViewModelListType::Outliner);
	if (OutlinerChildren)
	{
		SortChildren(*OutlinerChildren, SortingMode);
	}
}

void ISortableExtension::SortChildren(FViewModelChildren& Children, ESortingMode SortingMode)
{
	FViewModelHierarchyOperation Operation(Children.GetParent()->GetSharedData());

	struct FModelAndSortingKey
	{
		TSharedPtr<FViewModel> Model;
		FSortingKey SortingKey;

		static bool ComparePriorityFirst(const FModelAndSortingKey& A, const FModelAndSortingKey& B)
		{
			return FSortingKey::ComparePriorityFirst(A.SortingKey, B.SortingKey);
		}

		static bool CompareCustomOrderFirst(const FModelAndSortingKey& A, const FModelAndSortingKey& B)
		{
			return FSortingKey::CompareCustomOrderFirst(A.SortingKey, B.SortingKey);
		}
	};

	// Store all children in an array along with their computed sorting key, so that we can use
	// the core sorting algorithms and only query sorting keys once.
	uint32 NumSortable = 0;
	TArray<FModelAndSortingKey> ChildrenArray;
	for (TSharedPtr<FViewModel> Child : Children.IterateSubList())
	{
		FSortingKey SortingKey;
		if (ISortableExtension* Sortable = Child->CastThis<ISortableExtension>())
		{
			SortingKey = Sortable->GetSortingKey();
			++NumSortable;
		}
		ChildrenArray.Add(FModelAndSortingKey { Child, SortingKey });
	}

	// If there was no sorting information found, just leave the child list as is.
	if (NumSortable <= 1)
	{
		return;
	}

	// Sort!
	switch (SortingMode)
	{
		case ESortingMode::PriorityFirst:
		default:
			Algo::StableSort(ChildrenArray, FModelAndSortingKey::ComparePriorityFirst);
			break;
		case ESortingMode::CustomOrderFirst:
			Algo::StableSort(ChildrenArray, FModelAndSortingKey::CompareCustomOrderFirst);
			break;
	}

	// Put the first one at the head of the list
	Children.InsertChild(ChildrenArray[0].Model, nullptr);

	// Attach subsequent ones to the previous
	for (int32 Index = 1; Index < ChildrenArray.Num(); ++Index)
	{
		Children.InsertChild(ChildrenArray[Index].Model, ChildrenArray[Index-1].Model);
	}
}

} // namespace Sequencer
} // namespace UE

