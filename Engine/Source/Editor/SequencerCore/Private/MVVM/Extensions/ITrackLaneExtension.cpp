// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/ITrackLaneExtension.h"

#include "HAL/PlatformCrt.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Geometry.h"
#include "Math/RangeBound.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneTimeHelpers.h"
#include "TimeToPixel.h"
#include "Widgets/SWidget.h"

namespace UE
{
namespace Sequencer
{

FTrackLaneVerticalArrangement FTrackLaneVerticalAlignment::ArrangeWithin(float LayoutHeight) const
{
	const float AllowableHeight = LayoutHeight - VPadding*2.f;

	float DesiredHeight = 0.f;
	if (Mode == FTrackLaneVerticalAlignment::ESizeMode::Fixed)
	{
		DesiredHeight = VSizeParam;
	}
	else if (Mode == FTrackLaneVerticalAlignment::ESizeMode::Proportional)
	{
		DesiredHeight = AllowableHeight * VSizeParam;
	}

	if (VAlign == VAlign_Fill)
	{
		DesiredHeight = AllowableHeight;
	}

	const float FinalHeight = FMath::Min(DesiredHeight, AllowableHeight);

	float OffsetY = 0.f;
	switch(VAlign)
	{
	case VAlign_Center: OffsetY = (LayoutHeight-FinalHeight) * .5f;    break;
	case VAlign_Bottom: OffsetY = LayoutHeight-(VPadding+FinalHeight); break;
	default: break;
	}

	return FTrackLaneVerticalArrangement { OffsetY, FinalHeight };
}

TOptional<FFrameNumber> FTrackLaneVirtualAlignment::GetFiniteLength() const
{
	if (Range.GetLowerBound().IsClosed() && Range.GetUpperBound().IsClosed())
	{
		return FFrameNumber(UE::MovieScene::DiscreteSize(Range));
	}
	return TOptional<FFrameNumber>();
}

FTrackLaneScreenAlignment FTrackLaneVirtualAlignment::ToScreen(const FTimeToPixel& TimeToPixel, const FGeometry& ParentGeometry) const
{
	using namespace UE::MovieScene;

	if (!IsVisible())
	{
		return FTrackLaneScreenAlignment{};
	}

	const bool bHasStartFrame = Range.GetLowerBound().IsClosed();
	const bool bHasEndFrame   = Range.GetUpperBound().IsClosed();

	const float PixelStartX = bHasStartFrame ? TimeToPixel.FrameToPixel(DiscreteInclusiveLower(Range)) : 0.f;
	const float PixelEndX   = bHasEndFrame   ? TimeToPixel.FrameToPixel(DiscreteExclusiveUpper(Range)) : ParentGeometry.GetLocalSize().X;

	FTrackLaneScreenAlignment ScreenAlignment;
	ScreenAlignment.LeftPosPx = PixelStartX;
	ScreenAlignment.WidthPx   = PixelEndX-PixelStartX;
	ScreenAlignment.VerticalAlignment = VerticalAlignment;
	return ScreenAlignment;
}

FArrangedWidget FTrackLaneScreenAlignment::ArrangeWidget(TSharedRef<SWidget> InWidget, const FGeometry& ParentGeometry) const
{
	ensure(IsVisible());

	FTrackLaneVerticalArrangement VerticalLayout = VerticalAlignment.ArrangeWithin(ParentGeometry.GetLocalSize().Y);

	return ParentGeometry.MakeChild(
		InWidget,
		FVector2D(LeftPosPx, VerticalLayout.Offset),
		FVector2D(WidthPx,   VerticalLayout.Height)
	);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FTrackLaneScreenAlignment ITrackLaneWidget::GetAlignment(const ITrackLaneWidgetSpace& ScreenSpace, const FGeometry& InParentGeometry) const
{
	ensureMsgf(false, TEXT("FTrackLaneScreenAlignment GetAlignment(const FTimeToPixel&, const FGeometry&) is deprecated. Please update your code to override the new method"));
	return GetAlignment(ScreenSpace.GetScreenSpace(), InParentGeometry);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

} // namespace Sequencer
} // namespace UE

