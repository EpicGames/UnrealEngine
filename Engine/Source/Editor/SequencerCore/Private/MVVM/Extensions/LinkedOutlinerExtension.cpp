// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/LinkedOutlinerExtension.h"

#include "MVVM/Extensions/IOutlinerExtension.h"

namespace UE
{
namespace Sequencer
{

FLinkedOutlinerExtension::FLinkedOutlinerExtension()
{
}

TViewModelPtr<IOutlinerExtension> FLinkedOutlinerExtension::GetLinkedOutlinerItem() const
{
	return WeakModel.Pin();
}

void FLinkedOutlinerExtension::SetLinkedOutlinerItem(const TViewModelPtr<IOutlinerExtension>& InOutlinerItem)
{
	WeakModel = InOutlinerItem;
}

} // namespace Sequencer
} // namespace UE

