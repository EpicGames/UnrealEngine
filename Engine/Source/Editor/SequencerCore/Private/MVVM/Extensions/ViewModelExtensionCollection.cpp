// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/ViewModelExtensionCollection.h"

#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Templates/TypeHash.h"

namespace UE
{
namespace Sequencer
{

FViewModelExtensionCollection::FViewModelExtensionCollection(FViewModelTypeID InExtensionType)
	: ExtensionType(InExtensionType)
	, bNeedsUpdate(true)
{}

FViewModelExtensionCollection::FViewModelExtensionCollection(FViewModelTypeID InExtensionType, TWeakPtr<FViewModel> InWeakModel, int32 InDesiredRecursionDepth)
	: WeakModel(InWeakModel)
	, ExtensionType(InExtensionType)
	, DesiredRecursionDepth(InDesiredRecursionDepth)
	, bNeedsUpdate(true)
{
	TSharedPtr<FViewModel> Model = InWeakModel.Pin();
	if (Model && Model->IsConstructed())
	{
		Initialize();
	}
}

FViewModelExtensionCollection::~FViewModelExtensionCollection()
{
	// Clean up our subscription to HierarchyChanged events. We do not call Destroy
	// since that could call OnExtensionsDirtied which is a virtual function
	DestroyImpl();
}

void FViewModelExtensionCollection::Initialize()
{
	if (!OnHierarchyUpdatedHandle.IsValid())
	{
		if (TSharedPtr<FViewModel> Model = WeakModel.Pin())
		{
			TSharedPtr<FSharedViewModelData> SharedData = Model->GetSharedData();
			OnHierarchyUpdatedHandle = SharedData->SubscribeToHierarchyChanged(Model)
				.AddRaw(this, &FViewModelExtensionCollection::OnHierarchyUpdated);
		}
	}

	bNeedsUpdate = true;
	OnExtensionsDirtied();
}

void FViewModelExtensionCollection::Reinitialize(TWeakPtr<FViewModel> InWeakModel, int32 InDesiredRecursionDepth)
{
	Destroy();

	DesiredRecursionDepth = InDesiredRecursionDepth;
	WeakModel = InWeakModel;

	Initialize();
}

void FViewModelExtensionCollection::ConditionalUpdate() const
{
	TSharedPtr<FViewModel> Model = WeakModel.Pin();

	// If we have a valid model and are subscribed to a HierarchyChanged notification
	// make sure we report any outstanding operations that might affect the ptrs we cached
	if (Model && OnHierarchyUpdatedHandle.IsValid())
	{
		TSharedPtr<FSharedViewModelData> SharedData = Model->GetSharedData();
		if (SharedData)
		{
			SharedData->ReportLatentHierarchicalOperations();
		}
	}
	else if (ExtensionContainer.Num() != 0)
	{
		// Our model is no longer valid or we're not subscribed to notifications,
		// we cannot cache any ptrs because we can't keep them are up-to-date
		bNeedsUpdate = false;
		ExtensionContainer.Reset();
	}
	
	if (bNeedsUpdate)
	{
		// Our pointers could be out of date so update them now
		Update();
	}
}

void FViewModelExtensionCollection::FViewModelExtensionCollection::Update() const
{
	bNeedsUpdate = false;
	ExtensionContainer.Reset();

	TSharedPtr<FViewModel> Model = WeakModel.Pin();

	// Do not allow caching any ptrs from the tree unless we have a valid 
	// OnHierarchyUpdatedHandle that guarantees we can remain up-to-date
	if (Model && OnHierarchyUpdatedHandle.IsValid())
	{
		FParentFirstChildIterator ChildIt = Model->GetDescendants();
		if (DesiredRecursionDepth != -1)
		{
			ChildIt.SetMaxDepth(DesiredRecursionDepth);
		}

		for (; ChildIt; ++ChildIt)
		{
			if (void* Extension = ChildIt->CastRaw(ExtensionType))
			{
				ExtensionContainer.Add(Extension);
			}
		}
	}
}

void FViewModelExtensionCollection::OnHierarchyUpdated()
{
	bNeedsUpdate = true;
	OnExtensionsDirtied();
}

void FViewModelExtensionCollection::Destroy()
{
	DestroyImpl();

	bNeedsUpdate = true;
	OnExtensionsDirtied();
}

void FViewModelExtensionCollection::DestroyImpl()
{
	if (TSharedPtr<FViewModel> Model = WeakModel.Pin())
	{
		TSharedPtr<FSharedViewModelData> SharedData = Model->GetSharedData();
		if (SharedData)
		{
			SharedData->UnsubscribeFromHierarchyChanged(Model, OnHierarchyUpdatedHandle);
		}
	}

	OnHierarchyUpdatedHandle = FDelegateHandle();
}

} // namespace Sequencer
} // namespace UE

