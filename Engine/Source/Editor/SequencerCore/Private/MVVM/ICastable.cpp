// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ICastable.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/CastableTypeTable.h"
#include "HAL/IConsoleManager.h"

namespace UE::Sequencer
{

struct FCastableTypeInfo
{
	TMap<FName, void*> TypeNameToAllocation;

	~FCastableTypeInfo()
	{
		for (TTuple<FName, void*>& Pair : TypeNameToAllocation)
		{
			FMemory::Free(Pair.Value);
		}
	}

} GCastableTypeInfo;

TMap<FName, TUniquePtr<FCastableTypeTable>> GTypeNameToTypeTable;

void* ICastable::CastRaw(FViewModelTypeID InType)
{
	const void* Result = static_cast<const ICastable*>(this)->CastRaw(InType);
	return const_cast<void*>(Result);
}

const void* ICastable::CastRaw(FViewModelTypeID InType) const
{
	// Get the type info first to ensure that all necessary types are registered
	const FCastableTypeTable& TypeInfo = GetTypeTable();

	const void* Result = TypeInfo.Cast(this, InType.GetTypeID());
	if (!Result && DynamicTypes)
	{
		Result = DynamicTypes->CastDynamic(InType);
	}

	return Result;
}

FCastableTypeTable::FCastableTypeTable(uint8* InTypeMaskStorage)
	: TypeMask(InTypeMaskStorage)
{}

FCastableTypeTable* FCastableTypeTable::FCastableTypeTableGenerator::Commit(uint32 TypeID, FName InName)
{
	using namespace UE::MovieScene;

	uint8* NewType = (uint8*)FMemory::Malloc(ByteSize(), alignof(FCastableTypeTable));

	// FCastableTypeTable is a special type that uses a fixed allocation size determined by the size of the type table itself
	// There are 2 sets of data that live on the end of the struct: the sparse bitset buckets, and the type offsets.
	// Its layout in memory looks like this:
	//	|												sizeof(FCastableTypeTable)										|		sizeof(uint16)*NumStaticTypes		|		sizeof(uint16)*TypeMask.NumBuckets()	|
	//	|		int16*			|	  TFixedSparseBitSet	|		uint32		|		uint32		|		FName		|			uint16[NumStaticTypes]			|		uint16[TypeMask.NumBuckets()]			|
	//	|	StaticTypeOffsets	|		  TypeMask			|	NumStaticTypes	|	ThisTypeID		|	ThisTypeName	|											|		 										|
	//				\_________________________\________________________________________________________________________/										   /
	//										   \______________________________________________TypeMask.Buckets.Storage____________________________________________/
	uint8* DataOffset = NewType + sizeof(FCastableTypeTable);

	FCastableTypeTable* NewTable = new (NewType) FCastableTypeTable(DataOffset);
	NewTable->ThisTypeID = TypeID;
	NewTable->ThisTypeName = InName;

	TypeMask.CopyToUnsafe(NewTable->TypeMask, TypeMask.NumBuckets());

	DataOffset += TypeMask.NumBuckets()*sizeof(decltype(NewTable->TypeMask)::BucketType);

	// TypeOffset information lives after the bitset storage
	NewTable->StaticTypeOffsets = reinterpret_cast<int16*>(DataOffset);
	NewTable->NumStaticTypes = (uint32)StaticTypeOffsets.Num();
	FMemory::Memcpy(NewTable->StaticTypeOffsets, StaticTypeOffsets.GetData(), sizeof(int16)*StaticTypeOffsets.Num());

	checkf(!GCastableTypeInfo.TypeNameToAllocation.Contains(InName), TEXT("Type name %s has already been registered. This is not supported!"), *InName.ToString());
	GCastableTypeInfo.TypeNameToAllocation.Add(InName, NewType);

	return NewTable;
}

const FCastableTypeTable* FCastableTypeTable::FindTypeByName(FName InName)
{
	if (void* const * TypeInfo = GCastableTypeInfo.TypeNameToAllocation.Find(InName))
	{
		return reinterpret_cast<const FCastableTypeTable*>(*TypeInfo);
	}
	return nullptr;
}


} // namespace UE::Sequencer

