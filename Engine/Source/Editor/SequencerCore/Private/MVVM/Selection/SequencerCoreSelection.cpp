// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Selection/SequencerCoreSelection.h"


namespace UE::Sequencer
{

void FSequencerCoreSelection::Empty()
{
	FSelectionEventSuppressor Suppressor = SuppressEvents();
	for (FSelectionBase* Selection : SelectionSets)
	{
		Selection->Empty();
	}
}

FSelectionEventSuppressor FSequencerCoreSelection::SuppressEvents()
{
	return FSelectionEventSuppressor(this);
}

TUniquePtr<FSelectionEventSuppressor> FSequencerCoreSelection::SuppressEventsLongRunning()
{
	return MakeUnique<FSelectionEventSuppressor>(this);
}

void FSequencerCoreSelection::BroadcastSelectionChanged()
{
	TOptional<FSelectionEventSuppressor> Suppressor;

	TGuardValue<bool> TriggerGuard(bTriggeringSelectionEvents, true);

	const uint32 OldSerialNumber = SerialNumber;

	for (FSelectionBase* Selection : SelectionSets)
	{
		if (Selection->bSelectionChanged)
		{
			Selection->bSelectionChanged = false;

			// Only increment the serial number once for any number of buckets that changed
			if (OldSerialNumber == SerialNumber)
			{
				++SerialNumber;
				PreBroadcastChangeEvent();
			}

			// If a selection has changed, suppress events until the end of this function to prevent re-entrancy
			Suppressor.Emplace(this);

			Selection->OnChanged.Broadcast();

			if (!ensureMsgf(!Selection->bSelectionChanged, TEXT("Recursive event loop detected for selection changed event. Is something missing a check for IsTriggeringSelectionChangedEvents()?")))
			{
				Selection->bSelectionChanged = false;
			}
		}
	}

	if (OldSerialNumber != SerialNumber)
	{
		OnChanged.Broadcast();
	}
}

} // namespace UE::Sequencer