// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Selection/SequencerCoreSelectionTypes.h"
#include "MVVM/Selection/SequencerCoreSelection.h"

namespace UE::Sequencer
{

void FSelectionBase::Empty()
{
	FSelectionEventSuppressor SuppressEvents(Owner);
	EmptyImpl();
}

void FSelectionBase::ReportChanges(bool bInSelectionChanged)
{
	if (Owner)
	{
		if (bInSelectionChanged && !bSelectionChanged)
		{
			Owner->PreSelectionSetChangeEvent(this);
		}
		bSelectionChanged |= bInSelectionChanged;
	}
}

} // namespace UE::Sequencer