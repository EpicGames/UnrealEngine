// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"

namespace UE::Sequencer
{

bool FOutlinerSelection::OnSelectItem(const TWeakViewModelPtr<IOutlinerExtension>& WeakViewModel)
{
	if (TViewModelPtr<IOutlinerExtension> Outliner = WeakViewModel.Pin())
	{
		ISelectableExtension* Selectable = Outliner.AsModel()->CastThis<ISelectableExtension>();
		if (Selectable && Selectable->IsSelectable() == ESelectionIntent::Never)
		{
			return false;
		}

		Outliner->SetSelectionState(EOutlinerSelectionState::SelectedDirectly);
		return true;
	}

	return false;
}
void FOutlinerSelection::OnDeselectItem(const TWeakViewModelPtr<IOutlinerExtension>& WeakViewModel)
{
	if (TViewModelPtr<IOutlinerExtension> Outliner = WeakViewModel.Pin())
	{
		Outliner->SetSelectionState(EOutlinerSelectionState::None);
	}
}

} // namespace UE::Sequencer
