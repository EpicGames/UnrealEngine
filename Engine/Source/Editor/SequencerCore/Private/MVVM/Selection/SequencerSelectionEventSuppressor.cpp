// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"
#include "MVVM/Selection/SequencerCoreSelection.h"

namespace UE::Sequencer
{

FSelectionEventSuppressor::FSelectionEventSuppressor(FSequencerCoreSelection* InSelection)
	: Selection(InSelection)
{
	if (Selection)
	{
		++Selection->ScopedSuppressionCount;
	}
}

FSelectionEventSuppressor::~FSelectionEventSuppressor()
{
	if (Selection)
	{
		const int32 NewCount = --Selection->ScopedSuppressionCount;
		ensureMsgf(NewCount >= 0, TEXT("RAII Mismatch between scoped selection suppressors! Count reached %d"), NewCount);

		if (NewCount == 0)
		{
			Selection->BroadcastSelectionChanged();
		}
	}
}


} // namespace UE::Sequencer