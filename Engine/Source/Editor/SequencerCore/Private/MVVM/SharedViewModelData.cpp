// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/SharedViewModelData.h"

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/AssertionMacros.h"

namespace UE
{
namespace Sequencer
{

FSimpleMulticastDelegate& FSharedViewModelData::SubscribeToHierarchyChanged(const TSharedPtr<FViewModel>& InModel)
{
	return HierarchyChangedEventsByModel.FindOrAdd(InModel);
}

void FSharedViewModelData::UnsubscribeFromHierarchyChanged(const TSharedPtr<FViewModel>& InModel, FDelegateHandle InHandle)
{
	if (FSimpleMulticastDelegate* Event = HierarchyChangedEventsByModel.Find(InModel))
	{
		Event->Remove(InHandle);

		if (!Event->IsBound())
		{
			HierarchyChangedEventsByModel.Remove(InModel);
		}
	}
}

void FSharedViewModelData::UnsubscribeFromHierarchyChanged(const TSharedPtr<FViewModel>& InModel, FDelegateUserObjectConst InUserObject)
{
	if (FSimpleMulticastDelegate* Event = HierarchyChangedEventsByModel.Find(InModel))
	{
		Event->RemoveAll(InUserObject);

		if (!Event->IsBound())
		{
			HierarchyChangedEventsByModel.Remove(InModel);
		}
	}
}

void FSharedViewModelData::PurgeStaleHandlers()
{
	for (auto It = HierarchyChangedEventsByModel.CreateIterator(); It; ++It)
	{
		TSharedPtr<FViewModel> Model = It.Key().Pin();
		if (Model == nullptr || Model->GetSharedData().Get() != this)
		{
			It.RemoveCurrent();
		}
	}
}

void FSharedViewModelData::PreHierarchicalChange(const TSharedPtr<FViewModel>& InChangedModel)
{
	// If we do not have a current hierarchical operation, make a new one that will get flushed next tick
	if (!CurrentHierarchicalOperation)
	{
		LatentOperation = MakeUnique<FViewModelHierarchyOperation>(SharedThis(this));
	}

	check(CurrentHierarchicalOperation);
	CurrentHierarchicalOperation->PreHierarchicalChange(InChangedModel);
}

void FSharedViewModelData::BroadcastHierarchicalChange(const TSharedPtr<FViewModel>& InChangedModel)
{
	if (FSimpleMulticastDelegate* Event = HierarchyChangedEventsByModel.Find(InChangedModel))
	{
		Event->Broadcast();

		if (!Event->IsBound())
		{
			HierarchyChangedEventsByModel.Remove(InChangedModel);
		}
	}
}

void FSharedViewModelData::ReportLatentHierarchicalOperations()
{
	LatentOperation.Reset();
}

} // namespace Sequencer
} // namespace UE

