// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModelPtr.h"

namespace UE::Sequencer
{

TViewModelConversions<FViewModel>::operator TSharedPtr<FViewModel>() const
{
	return static_cast<const TViewModelPtr<FViewModel>*>(this)->Storage.GetModel();
}
TViewModelConversions<FViewModel>::operator TWeakPtr<FViewModel>() const
{
	return static_cast<const TViewModelPtr<FViewModel>*>(this)->Storage.GetModel();
}
TWeakViewModelPtr<FViewModel> TViewModelConversions<FViewModel>::AsWeak() const
{
	return *static_cast<const TViewModelPtr<FViewModel>*>(this);
}

TViewModelConversions<const FViewModel>::operator TSharedPtr<const FViewModel>() const
{
	return static_cast<const TViewModelPtr<const FViewModel>*>(this)->Storage.GetModel();
}
TViewModelConversions<const FViewModel>::operator TWeakPtr<const FViewModel>() const
{
	return static_cast<const TViewModelPtr<const FViewModel>*>(this)->Storage.GetModel();
}
TWeakViewModelPtr<const FViewModel> TViewModelConversions<const FViewModel>::AsWeak() const
{
	return *static_cast<const TViewModelPtr<const FViewModel>*>(this);
}

} // namespace UE::Sequencer

