// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModelTypeID.h"

namespace UE
{
namespace Sequencer
{

uint32 FViewModelTypeID::RegisterNewID()
{
	static uint32 ID = 0;
	return ID++;
}

} // namespace Sequencer
} // namespace UE

