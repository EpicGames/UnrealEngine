// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IDimmableExtension.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/IDraggableTrackAreaExtension.h"
#include "MVVM/Extensions/IDroppableExtension.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IGroupableExtension.h"
#include "MVVM/Extensions/IHotspotExtension.h"
#include "MVVM/Extensions/IHoveredExtension.h"
#include "MVVM/Extensions/IKeyExtension.h"
#include "MVVM/Extensions/ILayerBarExtension.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Extensions/IObjectModelExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/Extensions/IResizableExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"
#include "MVVM/Extensions/ISoloableExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/Extensions/IStretchableExtension.h"
#include "MVVM/Extensions/ITopTimeSliderOverlayExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ITrackAreaOverlayExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/LinkedOutlinerExtension.h"
#include "MVVM/Extensions/ITrackAreaViewSpaceProviderExtension.h"
#include "MVVM/Extensions/IViewSpaceClientExtension.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewSpace.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/EditorSharedViewModelData.h"
#include "MVVM/Views/ITrackAreaHotspot.h"

namespace UE
{
namespace Sequencer
{

// Interface types
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ICompoundOutlinerExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IDimmableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IDraggableOutlinerExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IDraggableTrackAreaExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IDroppableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IGeometryExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IGroupableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IHotspotExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IHoveredExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ILayerBarExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IDeactivatableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IMutableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ILockableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IOutlinerDropTargetOutlinerExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITrackAreaViewSpaceProviderExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IViewSpaceClientExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FViewSpaceClientExtensionShim);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IObjectModelExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IHierarchicalCache);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IOutlinerExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IPinnableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IRecyclableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IRenameableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IResizableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ISelectableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ISoloableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ISortableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IStretchableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITopTimeSliderOverlayExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITrackAreaExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITrackAreaOverlayExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITrackLaneExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IDeletableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IKeyExtension);

// Extension types
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IDynamicExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FDynamicExtensionContainer);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FLinkedOutlinerExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FDeactiveStateCacheExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FMuteStateCacheExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FSoloStateCacheExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FLockStateCacheExtension);

// View models
UE_SEQUENCER_DEFINE_CASTABLE(FEditorViewModel);
UE_SEQUENCER_DEFINE_CASTABLE(FOutlinerSpacer);
UE_SEQUENCER_DEFINE_CASTABLE(FOutlinerViewModel);
UE_SEQUENCER_DEFINE_CASTABLE(FTrackAreaViewModel);
UE_SEQUENCER_DEFINE_CASTABLE(FTrackAreaViewSpace);
UE_SEQUENCER_DEFINE_CASTABLE(FViewModel);
UE_SEQUENCER_DEFINE_CASTABLE(FSharedViewModelData);
UE_SEQUENCER_DEFINE_CASTABLE(FEditorSharedViewModelData);

// Views
UE_SEQUENCER_DEFINE_CASTABLE(ITrackAreaHotspot);

} // namespace Sequencer
} // namespace UE

