// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/EditorViewModel.h"

#include "CoreTypes.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/SharedViewModelData.h"
#include "Scripting/SequencerScriptingLayer.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"

namespace UE
{
namespace Sequencer
{

EViewModelListType FEditorViewModel::GetEditorPanelType()
{
	static EViewModelListType EditorPanelType = RegisterCustomModelListType();
	return EditorPanelType;
}

FEditorViewModel::FEditorViewModel()
	: PanelList(GetEditorPanelType())
{
	RegisterChildList(&PanelList);
}

FEditorViewModel::~FEditorViewModel()
{
}

void FEditorViewModel::InitializeEditor()
{
	ensureMsgf(!Outliner && !TrackArea, TEXT("This editor view-model has already been initialized"));

	PreInitializeEditorImpl();

	RootDataModel = CreateRootModelImpl();

	FViewModelChildren PanelChildren = GetEditorPanels();

	Outliner = CreateOutlinerImpl();
	PanelChildren.AddChild(Outliner);

	TrackArea = CreateTrackAreaImpl();
	PanelChildren.AddChild(TrackArea);

	TWeakPtr<FViewModel> WeakRootDataModel(RootDataModel);
	Outliner->Initialize(FWeakViewModelPtr(WeakRootDataModel));

	Selection = CreateSelectionImpl();

	USequencerScriptingLayer* Scripting = CreateScriptingLayerImpl();
	Scripting->Initialize(SharedThis(this));

	ScriptingLayer.Reset(Scripting);

	SetSharedData(MakeShared<FSharedViewModelData>());
	InitializeEditorImpl();
}

FViewModelPtr FEditorViewModel::GetRootModel() const
{
	return FViewModelPtr(RootDataModel);
}

FViewModelChildren FEditorViewModel::GetEditorPanels()
{
	return GetChildrenForList(&PanelList);
}

TSharedPtr<FOutlinerViewModel> FEditorViewModel::GetOutliner() const
{
	return Outliner;
}

TSharedPtr<FTrackAreaViewModel> FEditorViewModel::GetTrackArea() const
{
	return TrackArea;
}

TSharedPtr<FSequencerCoreSelection> FEditorViewModel::GetSelection() const
{
	return Selection;
}

USequencerScriptingLayer* FEditorViewModel::GetScriptingLayer() const
{
	return ScriptingLayer.Get();
}

TSharedPtr<FViewModel> FEditorViewModel::CreateRootModelImpl()
{
	return MakeShared<FViewModel>();
}

TSharedPtr<FOutlinerViewModel> FEditorViewModel::CreateOutlinerImpl()
{
	return MakeShared<FOutlinerViewModel>();
}

TSharedPtr<FTrackAreaViewModel> FEditorViewModel::CreateTrackAreaImpl()
{
	return MakeShared<FTrackAreaViewModel>();
}

TSharedPtr<FSequencerCoreSelection> FEditorViewModel::CreateSelectionImpl()
{
	struct FShellSelection : FSequencerCoreSelection
	{};
	return MakeShared<FShellSelection>();
}

USequencerScriptingLayer* FEditorViewModel::CreateScriptingLayerImpl()
{
	return NewObject<USequencerScriptingLayer>();
}

bool FEditorViewModel::IsReadOnly() const
{
	return false;
}

FViewDensityInfo FEditorViewModel::GetViewDensity() const
{
	return ViewDensity;
}

void FEditorViewModel::SetViewDensity(const FViewDensityInfo& InViewDensity)
{
	ViewDensity = InViewDensity;
}

} // namespace Sequencer
} // namespace UE

