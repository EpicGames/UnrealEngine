// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/AddOutlinerColumn.h"

namespace UE::Sequencer
{

FAddOutlinerColumn::FAddOutlinerColumn()
{
	Name     = FCommonOutlinerNames::Add;
	Label    = NSLOCTEXT("FAddOutlinerColumn", "AddColumnAdd", "Add");
	Position = FOutlinerColumnPosition{ 20, EOutlinerColumnGroup::Center };
	Layout   = FOutlinerColumnLayout{ 14.f, FMargin(4.f, 0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

} // namespace UE::Sequencer
