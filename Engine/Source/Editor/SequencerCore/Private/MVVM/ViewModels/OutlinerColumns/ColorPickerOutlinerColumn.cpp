// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/ColorPickerOutlinerColumn.h"

namespace UE::Sequencer
{

FColorPickerOutlinerColumn::FColorPickerOutlinerColumn()
{
	Name     = FCommonOutlinerNames::ColorPicker;
	Label    = NSLOCTEXT("FColorPickerOutlinerColumn", "ColorPickerColumnColorPicker", "Color Picker");
	Position = FOutlinerColumnPosition{ 30, EOutlinerColumnGroup::RightGutter };
	Layout   = FOutlinerColumnLayout{ 6.f, FMargin(0.f), HAlign_Fill, VAlign_Fill, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

} // namespace UE::Sequencer
