// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/DeactivateOutlinerColumn.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/Views/OutlinerColumns/SDeactivateColumnWidget.h"

#define LOCTEXT_NAMESPACE "FDeactivateOutlinerColumn"

namespace UE::Sequencer
{

FDeactivateOutlinerColumn::FDeactivateOutlinerColumn()
{
	Name     = FCommonOutlinerNames::Deactivate;
	Label    = LOCTEXT("DeactivateColumnLabel", "Deactivate");
	Position = FOutlinerColumnPosition{ 20, EOutlinerColumnGroup::AssetToggles };
	Layout   = FOutlinerColumnLayout{ 14, FMargin(4.f, 0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

bool FDeactivateOutlinerColumn::IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const
{
	if (const FDeactiveStateCacheExtension* const StateCache = InParams.OutlinerExtension.AsModel()->GetSharedData()->CastThis<FDeactiveStateCacheExtension>())
	{
		return EnumHasAnyFlags(StateCache->GetCachedFlags(InParams.OutlinerExtension)
			, ECachedDeactiveState::Deactivatable | ECachedDeactiveState::DeactivatableChildren);
	}

	return false;
}

TSharedPtr<SWidget> FDeactivateOutlinerColumn::CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow)
{
	return SNew(SDeactivateColumnWidget, SharedThis(this), InParams);
}
	
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
