// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/EditOutlinerColumn.h"

namespace UE::Sequencer
{

FEditOutlinerColumn::FEditOutlinerColumn()
{
	EOutlinerColumnFlags Flags = EOutlinerColumnFlags::OverflowSubsequentEmptyCells | EOutlinerColumnFlags::OverflowOnHover;

	Name     = FCommonOutlinerNames::Edit;
	Label    = NSLOCTEXT("FEditOutlinerColumn", "EditColumnEdit", "Edit");
	Position = FOutlinerColumnPosition{ 10, EOutlinerColumnGroup::Center };
	Layout   = FOutlinerColumnLayout{ 0.4f, FMargin(4.f, 1.f), HAlign_Fill, VAlign_Center, EOutlinerColumnSizeMode::Stretch, Flags };
}

} // namespace UE::Sequencer
