// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/KeyFrameOutlinerColumn.h"

namespace UE::Sequencer
{

FKeyFrameOutlinerColumn::FKeyFrameOutlinerColumn()
{
	Name     = FCommonOutlinerNames::KeyFrame;
	Label    = NSLOCTEXT("FKeyFrameOutlinerColumn", "ColumnName", "Key Frame");
	Position = FOutlinerColumnPosition{ 20, EOutlinerColumnGroup::RightGutter };
	Layout   = FOutlinerColumnLayout{ 34.f, FMargin(0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed };
}

bool FKeyFrameOutlinerColumn::IsColumnVisibleByDefault() const
{
	return false;
}

} // namespace UE::Sequencer
