// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/LabelOutlinerColumn.h"
#include "MVVM/Extensions/IOutlinerExtension.h"

#define LOCTEXT_NAMESPACE "FLabelOutlinerColumn"

namespace UE::Sequencer
{

FLabelOutlinerColumn::FLabelOutlinerColumn()
{
	EOutlinerColumnFlags Flags = EOutlinerColumnFlags::OverflowSubsequentEmptyCells | EOutlinerColumnFlags::OverflowOnHover | EOutlinerColumnFlags::Hidden;

	Name     = FCommonOutlinerNames::Label;
	Label    = LOCTEXT("LabelColumnLabel", "Label");
	Position = FOutlinerColumnPosition{ 0, EOutlinerColumnGroup::Center };
	Layout   = FOutlinerColumnLayout{ 0.6f, FMargin(4.f, 0.f), HAlign_Fill, VAlign_Center, EOutlinerColumnSizeMode::Stretch, Flags };
}

TSharedPtr<SWidget> FLabelOutlinerColumn::CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow)
{
	return InParams.OutlinerExtension->CreateOutlinerView(FCreateOutlinerViewParams{ TreeViewRow, InParams.Editor });
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
