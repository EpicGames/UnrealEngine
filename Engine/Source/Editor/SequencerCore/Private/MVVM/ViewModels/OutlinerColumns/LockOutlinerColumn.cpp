// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/LockOutlinerColumn.h"
#include "MVVM/Views/OutlinerColumns/SLockColumnWidget.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/SharedViewModelData.h"

#define LOCTEXT_NAMESPACE "FLockOutlinerColumn"

namespace UE::Sequencer
{

FLockOutlinerColumn::FLockOutlinerColumn()
{
	Name     = FCommonOutlinerNames::Lock;
	Label    = LOCTEXT("LockColumnLabel", "Lock");
	Position = FOutlinerColumnPosition{ 10, EOutlinerColumnGroup::AssetToggles };
	Layout   = FOutlinerColumnLayout{ 14, FMargin(4.f, 0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

bool FLockOutlinerColumn::IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const
{
	if (FLockStateCacheExtension* LockStateCache = InParams.OutlinerExtension.AsModel()->GetSharedData()->CastThis<FLockStateCacheExtension>())
	{
		return EnumHasAnyFlags(LockStateCache->GetCachedFlags(InParams.OutlinerExtension), ECachedLockState::Lockable | ECachedLockState::LockableChildren);
	}

	return false;
}

TSharedPtr<SWidget> FLockOutlinerColumn::CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow)
{
	return SNew(SLockColumnWidget, SharedThis(this), InParams);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE