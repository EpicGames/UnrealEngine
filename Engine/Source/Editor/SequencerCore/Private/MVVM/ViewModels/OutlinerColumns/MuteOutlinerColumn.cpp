// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/MuteOutlinerColumn.h"

#include "MVVM/SharedViewModelData.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Views/OutlinerColumns/SMuteColumnWidget.h"

#define LOCTEXT_NAMESPACE "FMuteOutlinerColumn"

namespace UE::Sequencer
{

FMuteOutlinerColumn::FMuteOutlinerColumn()
{
	Name     = FCommonOutlinerNames::Mute;
	Label    = LOCTEXT("MuteColumnLabel", "Mute");
	Position = FOutlinerColumnPosition{ 0, EOutlinerColumnGroup::TransientToggles };
	Layout   = FOutlinerColumnLayout{ 14, FMargin(4.f, 0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

bool FMuteOutlinerColumn::IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const
{
	if (FMuteStateCacheExtension* MuteStateCache = InParams.OutlinerExtension.AsModel()->GetSharedData()->CastThis<FMuteStateCacheExtension>())
	{
		return EnumHasAnyFlags(MuteStateCache->GetCachedFlags(InParams.OutlinerExtension), ECachedMuteState::Mutable | ECachedMuteState::MutableChildren);
	}

	return false;
}

TSharedPtr<SWidget> FMuteOutlinerColumn::CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow)
{
	return SNew(SMuteColumnWidget, SharedThis(this), InParams);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
