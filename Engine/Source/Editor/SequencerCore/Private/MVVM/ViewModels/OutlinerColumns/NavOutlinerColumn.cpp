// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/NavOutlinerColumn.h"

namespace UE::Sequencer
{

FNavOutlinerColumn::FNavOutlinerColumn()
{
	Name     = FCommonOutlinerNames::Nav;
	Label    = NSLOCTEXT("FNavOutlinerColumn", "ColumnName", "Key Navigation");
	Position = FOutlinerColumnPosition{ 10, EOutlinerColumnGroup::RightGutter };
	Layout   = FOutlinerColumnLayout{ 74.f, FMargin(0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed };
}

} // namespace UE::Sequencer
