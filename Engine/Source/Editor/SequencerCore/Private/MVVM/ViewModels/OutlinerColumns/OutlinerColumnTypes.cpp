// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"

namespace UE::Sequencer
{

FName FCommonOutlinerNames::Indicator("Indicator");
FName FCommonOutlinerNames::Pin("Pin");
FName FCommonOutlinerNames::Lock("Lock");
FName FCommonOutlinerNames::Deactivate("Deactivate");
FName FCommonOutlinerNames::Mute("Mute");
FName FCommonOutlinerNames::Solo("Solo");
FName FCommonOutlinerNames::Label("Label");
FName FCommonOutlinerNames::Edit("Edit");
FName FCommonOutlinerNames::Add("Add");
FName FCommonOutlinerNames::Nav("Nav");
FName FCommonOutlinerNames::KeyFrame("KeyFrame");
FName FCommonOutlinerNames::ColorPicker("ColorPicker");
FName FCommonOutlinerNames::TimeWarp("TimeWarp");
FName FCommonOutlinerNames::Condition("Condition");

} // namespace UE::Sequencer