// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/ISlateMetaData.h"
#include "Containers/Array.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"

namespace UE::Sequencer
{

struct FOutlinerHeaderRowWidgetMetaData : ISlateMetaData
{
	SLATE_METADATA_TYPE(FOutlinerHeaderRowWidgetMetaData, ISlateMetaData)

	TArray<FOutlinerColumnLayout> Columns;
};

} // namespace UE::Sequencer