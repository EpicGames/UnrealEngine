// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/SoloOutlinerColumn.h"
#include "MVVM/Views/OutlinerColumns/SSoloColumnWidget.h"
#include "MVVM/Extensions/ISoloableExtension.h"
#include "MVVM/SharedViewModelData.h"

#define LOCTEXT_NAMESPACE "FSoloOutlinerColumn"

namespace UE::Sequencer
{

FSoloOutlinerColumn::FSoloOutlinerColumn()
{
	Name     = FCommonOutlinerNames::Solo;
	Label    = LOCTEXT("SoloColumnLabel", "Solo");
	Position = FOutlinerColumnPosition{ 10, EOutlinerColumnGroup::TransientToggles };
	Layout   = FOutlinerColumnLayout{ 14, FMargin(4.f, 0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

bool FSoloOutlinerColumn::IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const
{
	if (FSoloStateCacheExtension* SoloStateCache = InParams.OutlinerExtension.AsModel()->GetSharedData()->CastThis<FSoloStateCacheExtension>())
	{
		return EnumHasAnyFlags(SoloStateCache->GetCachedFlags(InParams.OutlinerExtension), ECachedSoloState::Soloable | ECachedSoloState::SoloableChildren);
	}

	return false;
}

TSharedPtr<SWidget> FSoloOutlinerColumn::CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow)
{
	return SNew(SSoloColumnWidget, SharedThis(this), InParams);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE