// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerSpacer.h"

#include "HAL/PlatformCrt.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Math/NumericLimits.h"
#include "Misc/Attribute.h"
#include "SequencerCoreFwd.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

class FDragDropEvent;
class SWidget;

namespace UE
{
namespace Sequencer
{

FOutlinerSpacer::FOutlinerSpacer(float InDesiredSpacerHeight)
	: DesiredSpacerHeight(InDesiredSpacerHeight)
	, CustomOrder(-1)
{
}

FOutlinerSpacer::~FOutlinerSpacer()
{
}

bool FOutlinerSpacer::HasBackground() const
{
	return false;
}

FName FOutlinerSpacer::GetIdentifier() const
{
	return "Spacer";
}

FOutlinerSizing FOutlinerSpacer::GetOutlinerSizing() const
{
	FOutlinerSizing Sizing{ DesiredSpacerHeight };
	Sizing.Flags = EOutlinerSizingFlags::None;
	return Sizing;
}

TSharedPtr<SWidget> FOutlinerSpacer::CreateContextMenuWidget(const FCreateOutlinerContextMenuWidgetParams& InParams)
{
	return nullptr;
}

void FOutlinerSpacer::SortChildren()
{
	// Nothing to do
}

FSortingKey FOutlinerSpacer::GetSortingKey() const
{
	FSortingKey Key;
	Key.Priority = MAX_int8;
	Key.CustomOrder = CustomOrder;
	return Key;
}

void FOutlinerSpacer::SetCustomOrder(int32 InCustomOrder)
{
	CustomOrder = InCustomOrder;
}

TOptional<EItemDropZone> FOutlinerSpacer::CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone)
{
	TSharedPtr<FOutlinerViewModelDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FOutlinerViewModelDragDropOp>();
	if (!DragDropOp)
	{
		return TOptional<EItemDropZone>();
	}

	return EItemDropZone::AboveItem;
}

void FOutlinerSpacer::PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone)
{
	TViewModelPtr<IOutlinerDropTargetOutlinerExtension> Parent = FindAncestorOfType<IOutlinerDropTargetOutlinerExtension>();
	if (Parent)
	{
		Parent->PerformDrop(SharedThis(this), DragDropEvent, EItemDropZone::AboveItem);
	}
}

} // namespace Sequencer
} // namespace UE

