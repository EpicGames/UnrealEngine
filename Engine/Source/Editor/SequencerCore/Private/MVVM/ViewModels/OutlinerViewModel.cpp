// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerViewModel.h"

#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "MVVM/Extensions/IHoveredExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

class FDragDropOperation;

namespace UE
{
namespace Sequencer
{

FOutlinerViewModel::FOutlinerViewModel()
	: bRootItemsInvalidated(false)
{
}

void FOutlinerViewModel::Initialize(const FWeakViewModelPtr& InWeakRootDataModel)
{
	ensureMsgf(!WeakRootDataModel.Pin().IsValid(), TEXT("This outliner has already been initialized"));
	ensureMsgf(GetParent().IsValid(), TEXT("This outliner must be parented to an FEditorViewModel before being initialized"));
	
	WeakRootDataModel = InWeakRootDataModel;
	bRootItemsInvalidated = true;

	if (FViewModelPtr NewRootDataModel = WeakRootDataModel.Pin())
	{
		TSharedPtr<FSharedViewModelData> NewRootSharedData = NewRootDataModel->GetSharedData();
		NewRootSharedData->SubscribeToHierarchyChanged(NewRootDataModel)
			.AddSP(this, &FOutlinerViewModel::HandleDataHierarchyChanged);
	}
}

void FOutlinerViewModel::HandleDataHierarchyChanged()
{
	bRootItemsInvalidated = true;
	OnRefreshed.Broadcast();
}

TSharedPtr<FEditorViewModel> FOutlinerViewModel::GetEditor() const
{
	if (TSharedPtr<FViewModel> Parent = GetParent())
	{
		return Parent->CastThisSharedChecked<FEditorViewModel>();
	}
	return nullptr;
}

void FOutlinerViewModel::SetHoveredItem(TViewModelPtr<IOutlinerExtension> InHoveredItem)
{
	// Treat the bottom spacer item as empty space.
	if (InHoveredItem && InHoveredItem.AsModel()->IsA<FOutlinerSpacer>())
	{
		InHoveredItem = nullptr;
	}

	TViewModelPtr<IOutlinerExtension> CurrentHovered = GetHoveredItem();
	if (CurrentHovered == InHoveredItem)
	{
		return;
	}
	
	if (CurrentHovered)
	{
		if (TSharedPtr<IHoveredExtension> CurrentHoveredExtension = CurrentHovered.ImplicitCast())
		{
			CurrentHoveredExtension->OnUnhovered();
		}
	}

	WeakHoveredItem = InHoveredItem;

	if (TSharedPtr<IHoveredExtension> NewHoveredExtension = InHoveredItem.ImplicitCast())
	{
		NewHoveredExtension->OnHovered();
	}
}

TViewModelPtr<IOutlinerExtension> FOutlinerViewModel::GetHoveredItem() const
{
	return WeakHoveredItem.Pin();
}

TSharedPtr<SWidget> FOutlinerViewModel::CreateContextMenuWidget()
{
	return nullptr;
}

void FOutlinerViewModel::RequestUpdate()
{

}

FViewModelPtr FOutlinerViewModel::GetRootItem() const
{
	return WeakRootDataModel.Pin();
}

TArrayView<const TWeakViewModelPtr<IOutlinerExtension>> FOutlinerViewModel::GetTopLevelItems() const
{
	// Update the root items if they have been invalidated
	if (bRootItemsInvalidated)
	{
		FOutlinerViewModel* MutableThis = const_cast<FOutlinerViewModel*>(this);

		MutableThis->bRootItemsInvalidated = false;
		MutableThis->RootItems.Reset();

		TSharedPtr<FViewModel> Root = WeakRootDataModel.Pin();
		if (Root)
		{
			// Add the first outliner item in the tree per-branch
			for (TParentFirstChildIterator<IOutlinerExtension> It = Root->GetDescendantsOfType<IOutlinerExtension>(); It; ++It)
			{
				MutableThis->RootItems.Add(*It);
				It.IgnoreCurrentChildren();
			}
		}
	}
	return RootItems;
}

void FOutlinerViewModel::UnpinAllNodes()
{
	// @todo_sequencer_mvvm: pinning
	// const bool bIncludeRootNode = false;
	// RootNode->Traverse_ParentFirst([](FSequencerDisplayNode& InNode)
	// {
	// 	InNode.Unpin();
	// 	return true;
	// }, bIncludeRootNode);
}

TSharedRef<FDragDropOperation> FOutlinerViewModel::InitiateDrag(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedModels)
{
	FText DefaultText = FText::Format( NSLOCTEXT( "OutlinerViewModel", "DefaultDragDropFormat", "Move {0} item(s)" ), FText::AsNumber( InDraggedModels.Num() ) );
	return FOutlinerViewModelDragDropOp::New( MoveTemp(InDraggedModels), DefaultText, nullptr );
}

} // namespace Sequencer
} // namespace UE

