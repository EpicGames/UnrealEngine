// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"

#include "Internationalization/Internationalization.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/Attribute.h"
#include "SequencerCoreFwd.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "OutlinerViewModelDragDropOp"

namespace UE
{
namespace Sequencer
{

TSharedRef<FOutlinerViewModelDragDropOp> FOutlinerViewModelDragDropOp::New(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedViewModels, FText InDefaultText, const FSlateBrush* InDefaultIcon)
{
	TSharedRef<FOutlinerViewModelDragDropOp> NewOp = MakeShareable(new FOutlinerViewModelDragDropOp);

	NewOp->WeakViewModels = MoveTemp(InDraggedViewModels);
	NewOp->DefaultHoverText = NewOp->CurrentHoverText = InDefaultText;
	NewOp->DefaultHoverIcon = NewOp->CurrentIconBrush = InDefaultIcon;

	NewOp->Construct();
	return NewOp;
}

void FOutlinerViewModelDragDropOp::ResetToDefaultToolTip()
{
	CurrentHoverText = DefaultHoverText;
	CurrentIconBrush = DefaultHoverIcon;
}

void FOutlinerViewModelDragDropOp::Construct()
{
	FGraphEditorDragDropAction::Construct();

	SetFeedbackMessage(
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		.Content()
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 3.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &FOutlinerViewModelDragDropOp::GetDecoratorIcon)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock) 
				.Text(this, &FOutlinerViewModelDragDropOp::GetDecoratorText)
			]
		]
	);
}

TArrayView<const TWeakViewModelPtr<IOutlinerExtension>> FOutlinerViewModelDragDropOp::GetDraggedViewModels() const
{
	return WeakViewModels;
}

bool FOutlinerViewModelDragDropOp::ValidateParentChildDrop(const FViewModel& ProspectiveItem)
{
	for (const TViewModelPtr<IOutlinerExtension>& OutlinerItem : ProspectiveItem.GetAncestorsOfType<IOutlinerExtension>())
	{
		if (WeakViewModels.Contains(OutlinerItem))
		{
			CurrentHoverText = LOCTEXT("ParentIntoChildDragError", "Can't drag a parent into one of its children.");
			return false;
		}
	}
	return true;
}

void FOutlinerViewModelDragDropOp::GetTargetAndDropModel(const FViewModelPtr& CurrentViewModel, EItemDropZone InItemDropZone, FViewModelPtr& DropModel, FViewModelPtr& TargetModel) const
{
	DropModel = CurrentViewModel;

	// When dropping above or below an item, we always forward to its parent
	if (InItemDropZone == EItemDropZone::AboveItem || InItemDropZone == EItemDropZone::BelowItem)
	{
		TargetModel = DropModel;
		DropModel = DropModel->CastParent<IOutlinerDropTargetOutlinerExtension>();
	}
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE

