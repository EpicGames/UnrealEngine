// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/TrackAreaViewModel.h"

#include "HAL/PlatformCrt.h"
#include "ISequencerEditTool.h"
#include "MVVM/Extensions/ITrackAreaViewSpaceProviderExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "Misc/FrameRate.h"
#include "Templates/UnrealTemplate.h"

namespace UE
{
namespace Sequencer
{

FTrackAreaViewModel::FTrackAreaViewModel()
{
	bHotspotLocked = false;
	bEditToolLocked = false;
}

FTrackAreaViewModel::~FTrackAreaViewModel()
{
}

void FTrackAreaViewModel::OnConstruct()
{
	UpdateViewSpaces();
}

TSharedPtr<FEditorViewModel> FTrackAreaViewModel::GetEditor() const
{
	if (TSharedPtr<FViewModel> Parent = GetParent())
	{
		return Parent->CastThisSharedChecked<FEditorViewModel>();
	}
	return nullptr;
}

void FTrackAreaViewModel::UpdateViewSpaces()
{
	ViewSpaces.Empty();

	FViewModelPtr RootModel = GetEditor()->GetRootModel();
	FSimpleMulticastDelegate& Event = RootModel->GetSharedData()->SubscribeToHierarchyChanged(RootModel);
	if (!Event.IsBoundToObject(this))
	{
		Event.AddSP(this, &FTrackAreaViewModel::UpdateViewSpaces);
	}

	for (TViewModelPtr<ITrackAreaViewSpaceProviderExtension> ViewSpace : RootModel->GetChildrenOfType<ITrackAreaViewSpaceProviderExtension>())
	{
		ViewSpace->UpdateViewSpaces(*this);
	}
}

void FTrackAreaViewModel::AddEditTool(TSharedPtr<ISequencerEditTool> InNewTool)
{
	EditTools.Add(MoveTemp(InNewTool));
}

FTimeToPixel FTrackAreaViewModel::GetTimeToPixel(float TrackAreaWidth) const
{
	return FTimeToPixel(TrackAreaWidth, GetViewRange(), GetTickResolution());
}

FFrameRate FTrackAreaViewModel::GetTickResolution() const
{
	return FFrameRate();
}

TRange<double> FTrackAreaViewModel::GetViewRange() const
{
	return TRange<double>::Empty();
}

TSharedPtr<ITrackAreaHotspot> FTrackAreaViewModel::GetHotspot() const
{
	return HotspotStack.Num() > 0 ? HotspotStack[0] : nullptr;
}

void FTrackAreaViewModel::SetHotspot(TSharedPtr<ITrackAreaHotspot> NewHotspot)
{
	// We always set a hotspot if there is none
	if (HotspotStack.Num() == 0 || bHotspotLocked == false)
	{
		HotspotStack.Empty();

		if (NewHotspot)
		{
			HotspotStack.Push(NewHotspot);

			// Simulate an update-on-hover for the new hotspot to ensure that any hover behavior doesn't have to wait until the next frame
			NewHotspot->UpdateOnHover(*this);
		}

		OnHotspotChangedDelegate.Broadcast(NewHotspot);
	}
}

void FTrackAreaViewModel::AddHotspot(TSharedPtr<ITrackAreaHotspot> NewHotspot)
{
	// We always set a hotspot if there is none
	if (bHotspotLocked == false)
	{
		TSharedPtr<ITrackAreaHotspot> PreviousHotspot = GetHotspot();

		HotspotStack.Push(NewHotspot);
		HotspotStack.Sort([](TSharedPtr<ITrackAreaHotspot> A, TSharedPtr<ITrackAreaHotspot> B){
			return A->Priority() > B->Priority();
		});

		// Simulate an update-on-hover for the new hotspot to ensure that any hover behavior doesn't have to wait until the next frame
		NewHotspot->UpdateOnHover(*this);

		TSharedPtr<ITrackAreaHotspot> CurrentHotspot = GetHotspot();
		if (PreviousHotspot != CurrentHotspot)
		{
			OnHotspotChangedDelegate.Broadcast(CurrentHotspot);
		}
	}
}

void FTrackAreaViewModel::RemoveHotspot(FViewModelTypeID Type)
{
	if (bHotspotLocked == false)
	{
		TSharedPtr<ITrackAreaHotspot> PreviousHotspot = GetHotspot();

		HotspotStack.RemoveAll([Type](TSharedPtr<ITrackAreaHotspot> In){
			return In->CastRaw(Type) != nullptr;
		});

		TSharedPtr<ITrackAreaHotspot> CurrentHotspot = GetHotspot();
		if (PreviousHotspot != CurrentHotspot)
		{
			OnHotspotChangedDelegate.Broadcast(CurrentHotspot);
		}
	}
}

void FTrackAreaViewModel::ClearHotspots()
{
	if (bHotspotLocked == false)
	{
		HotspotStack.Empty();
		OnHotspotChangedDelegate.Broadcast(nullptr);
	}
}

void FTrackAreaViewModel::LockHotspot(bool bIsLocked)
{
	bHotspotLocked = bIsLocked;
}

FOnTrackAreaHotspotChanged& FTrackAreaViewModel::GetOnHotspotChangedDelegate()
{
	return OnHotspotChangedDelegate;
}

bool FTrackAreaViewModel::CanActivateEditTool(FName Identifier) const
{
	auto IdentifierIsFound = [=](TSharedPtr<ISequencerEditTool> InEditTool){ return InEditTool->GetIdentifier() == Identifier; };
	if (bEditToolLocked)
	{
		return false;
	}
	else if (!EditTool.IsValid())
	{
		return EditTools.ContainsByPredicate(IdentifierIsFound);
	}
	// Can't activate a tool that's already active
	else if (EditTool->GetIdentifier() == Identifier)
	{
		return false;
	}
	// Can only activate a new tool if the current one will let us
	else
	{
		return EditTool->CanDeactivate() && EditTools.ContainsByPredicate(IdentifierIsFound);
	}
}

bool FTrackAreaViewModel::AttemptToActivateTool(FName Identifier)
{
	if ( CanActivateEditTool(Identifier) )
	{
		EditTool = *EditTools.FindByPredicate([=](TSharedPtr<ISequencerEditTool> InEditTool){ return InEditTool->GetIdentifier() == Identifier; });
		return true;
	}

	return false;
}

void FTrackAreaViewModel::LockEditTool()
{
	bEditToolLocked = true;
}

void FTrackAreaViewModel::UnlockEditTool()
{
	bEditToolLocked = false;
}

void FTrackAreaViewModel::AddViewSpace(const FGuid& InID, TSharedPtr<FTrackAreaViewSpace> ViewSpace)
{
	TSharedPtr<FTrackAreaViewSpace> Existing = ViewSpaces.FindRef(InID);
	if (Existing != ViewSpace)
	{
		ViewSpaces.Add(InID, ViewSpace);
		OnViewSpaceAdded.Broadcast(InID, ViewSpace);
	}
}

void FTrackAreaViewModel::RemoveViewSpace(const FGuid& InID)
{
	TSharedPtr<FTrackAreaViewSpace> Existing = ViewSpaces.FindRef(InID);
	if (Existing)
	{
		ViewSpaces.Remove(InID);
		OnViewSpaceRemoved.Broadcast(InID, Existing);
	}
}

TSharedPtr<FTrackAreaViewSpace> FTrackAreaViewModel::FindViewSpace(const FGuid& InID) const
{
	return ViewSpaces.FindRef(InID);
}

} // namespace Sequencer
} // namespace UE

