// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/TrackAreaViewSpace.h"

namespace UE::Sequencer
{


FTrackAreaViewSpace::FTrackAreaViewSpace()
{
}

FTrackAreaViewSpace::~FTrackAreaViewSpace()
{
}

double FTrackAreaViewSpace::SourceToView(double Seconds) const
{
	return Seconds;
}

double FTrackAreaViewSpace::ViewToSource(double Source) const
{
	return Source;
}


} // namespace UE::Sequencer

