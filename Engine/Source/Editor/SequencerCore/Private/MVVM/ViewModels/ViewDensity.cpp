// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/ViewDensity.h"
#include "HAL/IConsoleManager.h"

namespace UE::Sequencer
{

float GSequencerOutlinerCompactHeight = 22.f;
FAutoConsoleVariableRef CVarSequencerOutlinerCompactHeight(
	TEXT("Sequencer.Outliner.CompactHeight"),
	GSequencerOutlinerCompactHeight,
	TEXT("(Default: 22.f. Defines the height of outliner items when in compact mode.")
	);
float GSequencerOutlinerRelaxedHeight = 28.f;
FAutoConsoleVariableRef CVarSequencerOutlinerRelaxedHeight(
	TEXT("Sequencer.Outliner.RelaxedHeight"),
	GSequencerOutlinerRelaxedHeight,
	TEXT("(Default: 28.f. Defines the height of outliner items when in relaxed mode.")
	);


FViewDensityInfo::FViewDensityInfo()
	: UniformHeight(GetUniformHeight(EViewDensity::Compact))
	, Density(EViewDensity::Compact)
{}

FViewDensityInfo::FViewDensityInfo(EViewDensity InViewDensity)
	: UniformHeight(GetUniformHeight(InViewDensity))
	, Density(InViewDensity)
{}

TOptional<float> FViewDensityInfo::GetUniformHeight(EViewDensity Density)
{
	switch(Density)
	{
		case EViewDensity::Compact:  return GSequencerOutlinerCompactHeight;
		case EViewDensity::Relaxed:  return GSequencerOutlinerRelaxedHeight;
		default:                     return TOptional<float>();
	}
}


} // namespace UE::Sequencer