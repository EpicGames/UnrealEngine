// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/ViewModel.h"

#include "HAL/PlatformCrt.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"

namespace UE
{
namespace Sequencer
{

uint32 GNextViewModelID = 0;

FViewModel::FViewModel()
	: FirstChildListHead(nullptr)
	, ActiveIterationCount(0)
	, bNeedsConstruction(1)
{
	// Don't really care about overflow here - by the time it does overflow, any overlapping IDs should have been released
	ModelID = GNextViewModelID++;
}

FViewModel::~FViewModel()
{
	ensureAlwaysMsgf(ActiveIterationCount == 0, TEXT("Attempting to make hierarchical changes to a view model while an iteration is active"));
}

bool FViewModel::IsConstructed() const
{
	return SharedData != nullptr && bNeedsConstruction == false;
}

void FViewModel::RegisterChildList(FViewModelListHead* InChildren)
{
	checkf(InChildren->NextListHead == nullptr, TEXT("Cannot register a child list that has attached child lists itself"));
	checkf(InChildren->GetHead() == nullptr, TEXT("Cannot attach a child list that has been populated with children"));

	if (FirstChildListHead)
	{
		if (FirstChildListHead->NextListHead)
		{
			InChildren->NextListHead = FirstChildListHead->NextListHead;
		}
		FirstChildListHead->NextListHead = InChildren;
	}
	else
	{
		FirstChildListHead = InChildren;
	}
}

TOptional<FViewModelChildren> FViewModel::FindChildList(EViewModelListType InFilter)
{
	FViewModelListHead* CurrentList = FirstChildListHead;

	// Skip lists that do not match the filter
	while (CurrentList && !EnumHasAnyFlags(CurrentList->Type, InFilter))
	{
		CurrentList = CurrentList->NextListHead;
	}

	if (CurrentList)
	{
		return FViewModelChildren(AsShared(), CurrentList);
	}
	else
	{
		return TOptional<FViewModelChildren>();
	}
}

FViewModelChildren FViewModel::GetChildList(EViewModelListType InType)
{
	TOptional<FViewModelChildren> Result = FindChildList(InType);
	check(Result.IsSet());
	return Result.GetValue();
}

FViewModelPtr FViewModel::GetParent() const
{
	return WeakParent.Pin();
}

FViewModelPtr FViewModel::GetRoot() const
{
	TSharedPtr<FViewModel> CurrentItem(SharedThis(const_cast<FViewModel*>(this)));
	while (CurrentItem->GetParent())
	{
		CurrentItem = CurrentItem->GetParent();
	}
	return CurrentItem;
}

FViewModel::ESetParentResult FViewModel::SetParentOnly(const TSharedPtr<FViewModel>& NewParent, bool bReportChanges)
{
	ensureAlwaysMsgf(ActiveIterationCount == 0, TEXT("Attempting to make hierarchical changes to a view model while an iteration is active"));

	TSharedPtr<FViewModel> OldParent = WeakParent.Pin();
	if (OldParent == NewParent)
	{
		return ESetParentResult::AlreadySameParent;
	}

	if (bReportChanges && SharedData)
	{
		SharedData->PreHierarchicalChange(AsShared());
	}

	if (NewParent)
	{
		// If it's different shared data, report the change on the new shared data as well
		if (bReportChanges && NewParent->SharedData && NewParent->SharedData != SharedData)
		{
			NewParent->SharedData->PreHierarchicalChange(AsShared());
		}

		WeakParent = NewParent;
		SetSharedData(NewParent->SharedData);

		return ESetParentResult::ChangedParent;
	}
	else
	{
		WeakParent = nullptr;
		SetSharedData(nullptr);

		return ESetParentResult::ClearedParent;
	}
}

FViewModelChildren FViewModel::GetChildrenForList(FViewModelListHead* ListHead)
{
	return FViewModelChildren(AsShared(), ListHead);
}

void FViewModel::RemoveFromParent()
{
	FViewModelHierarchyOperation Operation(SharedData);

	Link.Unlink();

	// When this model is removed from any of its parent's lists, it is detached from that parent.
	SetParentOnly(nullptr);
}

void FViewModel::DiscardAllChildren()
{
	FViewModelHierarchyOperation Operation(SharedData);

	// Remove parent -> child relationships
	for (TSharedPtr<FViewModel> Child : GetChildren().ToArray())
	{
		Child->SetParentOnly(nullptr);
	}

	// Unlink the head of all our child lists
	FViewModelListHead* CurrentHead = FirstChildListHead;
	while (CurrentHead)
	{
		CurrentHead->ReliquishList();
		CurrentHead = CurrentHead->NextListHead;
	}
}

TSharedPtr<FSharedViewModelData> FViewModel::GetSharedData() const
{
	return SharedData;
}

void FViewModel::SetSharedDataFromConstructor(TSharedPtr<FSharedViewModelData> InSharedData)
{
	SharedData = InSharedData;
}

void FViewModel::SetSharedData(TSharedPtr<FSharedViewModelData> InSharedData)
{
	if (SharedData == InSharedData)
	{
		return;
	}

	if (InSharedData)
	{
		bNeedsConstruction = true;
		SharedData = InSharedData;

		for (const FViewModelPtr& Child : GetDescendants())
		{
			Child->bNeedsConstruction = true;
			Child->SharedData = InSharedData;
		}

		OnConstruct();
		bNeedsConstruction = false;

		for (const FViewModelPtr& Child : GetDescendants())
		{
			if (Child->bNeedsConstruction)
			{
				// This is safe to call because we're inside a parent-first iterator.
				// Any children that are added inside here will get implicitly
				// included by the iterator, but they may have already been constructed
				Child->OnConstruct();
				Child->bNeedsConstruction = false;
			}
		}
	}
	else
	{
		SharedData = nullptr;

		for (const FViewModelPtr& Child : GetDescendants())
		{
			Child->SharedData = nullptr;
		}

		OnDestruct();

		for (const FViewModelPtr& Child : GetDescendants())
		{
			// This is safe to call because we're inside a parent-first iterator.
			// Any children that are removed inside here will get implicitly
			// excluded by the iterator
			Child->OnDestruct();
		}
	}
}

bool FViewModel::HasChildren() const
{
	const FViewModelListHead* CurrentHead = FirstChildListHead;
	while (CurrentHead)
	{
		if (CurrentHead->GetHead())
		{
			return true;
		}
		CurrentHead = CurrentHead->NextListHead;
	}
	return false;
}

TSharedPtr<FViewModel> FViewModel::GetPreviousSibling() const
{
	// This is a little convoluted and is sub-optimal due to FViewModelListLink not knowing its owner

	// PreviousLink is either a ptr to our previous sibling's FViewModel::Link, or a parent's FViewModelListHead::HeadLink
	TSharedPtr<FViewModelListLink> PreviousLink = Link.WeakPrev.Pin();
	// PreviousPreviousLink is null if we are the first child (since FViewModelListHead::HeadLink never has a valid previous)
	// Or is a valid pointer to its previous
	TSharedPtr<FViewModelListLink> PreviousPreviousLink = PreviousLink ? PreviousLink->WeakPrev.Pin() : nullptr;

	return PreviousPreviousLink ? PreviousPreviousLink->Next : nullptr;
}

TSharedPtr<FViewModel> FViewModel::GetNextSibling() const
{
	return Link.Next;
}

FViewModelListIterator FViewModel::GetChildren(EViewModelListType InFilter) const
{
	return FViewModelListIterator{ FirstChildListHead, InFilter };
}

void FViewModel::GetDescendantsOfType(FViewModelTypeID Type, TArray<TSharedPtr<FViewModel>>& OutChildren, EViewModelListType InFilter) const
{
	for (TSharedPtr<FViewModel> Child : GetDescendants(false, InFilter))
	{
		if (Child->CastRaw(Type) != nullptr)
		{
			OutChildren.Add(Child);
		}
	}
}

FParentFirstChildIterator FViewModel::GetDescendants(bool bIncludeThis, EViewModelListType InFilter) const
{
	return FParentFirstChildIterator{ SharedThis(const_cast<FViewModel*>(this)), bIncludeThis, InFilter };
}

FParentModelIterator FViewModel::GetAncestors(bool bIncludeThis) const
{
	return FParentModelIterator(SharedThis(const_cast<FViewModel*>(this)), bIncludeThis);
}

FViewModelPtr FViewModel::FindAncestorOfType(FViewModelTypeID Type, bool bIncludeThis) const
{
	TSharedPtr<FViewModel> Parent = bIncludeThis ? SharedThis(const_cast<FViewModel*>(this)) : GetParent();
	while (Parent)
	{
		if (Parent->CastRaw(Type) != nullptr)
		{
			return Parent;
		}
		Parent = Parent->GetParent();
	}
	return nullptr;
}

FViewModelPtr FViewModel::FindAncestorOfTypes(TArrayView<const FViewModelTypeID> Types, bool bIncludeThis) const
{
	TSharedPtr<FViewModel> Parent = bIncludeThis ? SharedThis(const_cast<FViewModel*>(this)) : GetParent();
	while (Parent)
	{
		bool bSupportsAllTypes = true;
		for (const FViewModelTypeID Type : Types)
		{
			if (Parent->CastRaw(Type) == nullptr)
			{
				bSupportsAllTypes = false;
				break;
			}
		}
		if (bSupportsAllTypes)
		{
			return Parent;
		}
		Parent = Parent->GetParent();
	}
	return nullptr;
}

} // namespace Sequencer
} // namespace UE

