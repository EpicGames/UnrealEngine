// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/SharedViewModelData.h"
#include "Templates/Greater.h"

namespace UE
{
namespace Sequencer
{

EViewModelListType RegisterCustomModelListType()
{
	static EViewModelListType NextCustom = EViewModelListType::Custom;
	EViewModelListType Result = NextCustom;

	NextCustom = (EViewModelListType)((uint32)NextCustom << 1);
	check(NextCustom != EViewModelListType::Recycled);

	return Result;
}

FViewModelListLink::~FViewModelListLink()
{
	Unlink();
}

void FViewModelListLink::Unlink()
{
	TSharedPtr<FViewModelListLink> PreviousLink = WeakPrev.Pin();
	if (PreviousLink)
	{
		// Our next is now PreviousLink's next
		PreviousLink->Next = Next;
		if (Next)
		{
			Next->Link.WeakPrev = WeakPrev;
			Next = nullptr;
		}
		WeakPrev = nullptr;
	}
	else if (Next)
	{
		Next->Link.WeakPrev = nullptr;
		Next = nullptr;
	}
}

TSharedPtr<FViewModel> FViewModelListLink::FindLastLink() const
{
	TSharedPtr<FViewModel> Last = Next;
	while (Last && Last->Link.Next)
	{
		Last = Last->Link.Next;
	}
	return Last;
}

void FViewModelListLink::LinkModelTo(TSharedPtr<FViewModel> Model, TSharedPtr<FViewModelListLink> ToLink)
{
	ensureAlwaysMsgf(Model->ActiveIterationCount == 0, TEXT("Attempting to make hierarchical changes to a view model while an iteration is active"));

	// Unlink the model from its previous chain
	Model->Link.Unlink();

	// Attach it to the new chain (possibly the same one, in a different spot)
	Model->Link.Next = ToLink->Next;
	if (ToLink->Next)
	{
		ToLink->Next->Link.WeakPrev = TSharedPtr<FViewModelListLink>(Model, &Model->Link);
	}

	ToLink->Next = Model;
	Model->Link.WeakPrev = ToLink;

	DetectLinkListCycle(Model);
}

void FViewModelListLink::DetectLinkListCycle()
{
	DetectLinkListCycle(Next);
}

void FViewModelListLink::DetectLinkListCycle(TSharedPtr<FViewModel> StartAt)
{
#if UE_SEQUENCER_DETECT_LINK_LIST_CYCLES
	// Use Floyd's "tortoise-and-hare" cycle finding algorithm.
	if (StartAt)
	{
		// Start the tortoise and the hare on the following item. Make the hare go twice as fast
		// as the tortoise, until it reaches the tail of the list, or until we detect a cycle as
		// it loops back on the tortoise.
		TSharedPtr<FViewModel> Tortoise = StartAt->Link.Next;
		TSharedPtr<FViewModel> Hare = Tortoise ? Tortoise->Link.Next : nullptr;
		while (Hare && Tortoise != Hare)
		{
			Tortoise = Tortoise->Link.Next;
			Hare = Hare->Link.Next;
			if (Hare)
			{
				Hare = Hare->Link.Next;
			}
		}
		// If we reached the end, there's no loop.
		if (!Hare)
		{
			return;
		}

		// Loop detected! Find the start of the loop.
		int32 CycleStartIndex = 0;
		Tortoise = StartAt;
		while (Tortoise != Hare)
		{
			Tortoise = Tortoise->Link.Next;
			Hare = Hare->Link.Next;
			++CycleStartIndex;
		}
		TSharedPtr<FViewModel> CycleStart = Tortoise;

		// And now find the length of the cycle.
		int32 CycleLength = 1;
		Hare = Tortoise->Link.Next;
		while (Tortoise != Hare)
		{
			Hare = Hare->Link.Next;
			++CycleLength;
		}

		ensureAlwaysMsgf(false, TEXT("Cycle detected! Starts at %" UPTRINT_x_FMT " (index %d) and is %d items long"),
				CycleStart.Get(), CycleStartIndex, CycleLength);
	}
#endif  // UE_SEQUENCER_DETECT_LINK_LIST_CYCLES
}

FViewModelPtr FViewModelListHead::GetHead() const
{
	return HeadLink.Next;
}

/** Finds the last model of this list */
FViewModelPtr FViewModelListHead::GetTail() const
{
	return HeadLink.FindLastLink();
}

FViewModelSubListIterator FViewModelListHead::Iterate() const
{
	return FViewModelSubListIterator(this);
}

TSharedPtr<FViewModel> FViewModelListHead::ReliquishList()
{
	// Reassign the destination head
	TSharedPtr<FViewModel> Head = GetHead();
	if (Head)
	{
		Head->Link.WeakPrev = nullptr;
	}
	HeadLink.Next = nullptr;

	return Head;
}

FScopedViewModelListHead::FScopedViewModelListHead(const TSharedPtr<FViewModel>& InModel, EViewModelListType InType)
	: Model(InModel)
	, ListHead(InType)
{
	check(InModel);
	InModel->RegisterChildList(&ListHead);
}

FScopedViewModelListHead::~FScopedViewModelListHead()
{
	// Destroy everything in this list
	GetChildren().Empty();

	FViewModelListHead* ThisList = &ListHead;
	FViewModelListHead** Ptr = &Model->FirstChildListHead;
	while (*Ptr)
	{
		if (*Ptr == ThisList)
		{
			*Ptr = ThisList->NextListHead;
			ThisList->NextListHead = nullptr;
			break;
		}
		Ptr = &(*Ptr)->NextListHead;
	}
}

FViewModelChildren FScopedViewModelListHead::GetChildren()
{
	return FViewModelChildren(Model, &ListHead);
}

FViewModelChildren::FViewModelChildren(const TSharedPtr<FViewModel>& InOwner, FViewModelListHead* InListHead)
	: Owner(InOwner)
	, ListHead(InListHead)
{}

EViewModelListType FViewModelChildren::GetType() const
{
	return ListHead->Type;
}

FViewModelPtr FViewModelChildren::GetParent() const
{
	return Owner;
}

FViewModelPtr FViewModelChildren::GetHead() const
{
	return ListHead->GetHead();
}

FViewModelPtr FViewModelChildren::GetTail() const
{
	return ListHead->GetTail();
}

FViewModelSubListIterator FViewModelChildren::IterateSubList() const
{
	return FViewModelSubListIterator(ListHead);
}

void FViewModelChildren::AddChild(const TSharedPtr<FViewModel>& Child)
{
	if (!ensureMsgf(Child != Owner, TEXT("Cannot attach a model to itself")))
	{
		return;
	}
	if (!ensureMsgf(Child, TEXT("Unable to add a null child")))
	{
		return;
	}

	if (Child->SetParentOnly(Owner) == FViewModel::ESetParentResult::AlreadySameParent)
	{
		// If we did not re-assign the parent, we need to potentially report a hierarchical change
		// This is really just an optimization to prevent us calling PreHierarchicalChange too often
		if (Child->SharedData)
		{
			Child->SharedData->PreHierarchicalChange(Child);
		}
	}

	FViewModelListLink::LinkModelTo(Child, TSharedPtr<FViewModelListLink>(Owner, &ListHead->HeadLink));
}

void FViewModelChildren::InsertChild(const TSharedPtr<FViewModel>& Child, const TSharedPtr<FViewModel>& PreviousSibling)
{
	if (!ensureMsgf(Child, TEXT("Unable to add a null child")))
	{
		return;
	}

	if (!ensureMsgf(PreviousSibling == nullptr || PreviousSibling->GetParent() == Owner, TEXT("Specified PreviousSibling does not belong to this child list")))
	{
		return;
	}

	// Target link is either the previous sibling, or the owner's list head
	TSharedPtr<FViewModelListLink> TargetLink = PreviousSibling
		? TSharedPtr<FViewModelListLink>(PreviousSibling, &PreviousSibling->Link)
		: TSharedPtr<FViewModelListLink>(Owner, &ListHead->HeadLink);

	if (TargetLink->Next != Child)
	{
		// Ensure we inherit the same parent
		if (Child->SetParentOnly(Owner) == FViewModel::ESetParentResult::AlreadySameParent && Child->SharedData)
		{
			// If we did not re-assign the parent, we need to potentially report a hierarchical change
			// This is really just an optimization to prevent us calling PreHierarchicalChange too often
			Child->SharedData->PreHierarchicalChange(Child);
		}

		// Link to the specified previous object
		FViewModelListLink::LinkModelTo(Child, TargetLink);
	}
}

void FViewModelChildren::MoveChildrenTo(const FViewModelChildren& OutDestination)
{
	if (IsEmpty())
	{
		return;
	}

	// Probably an error if you're trying to do a self-assignment
	if (!ensure(ListHead != OutDestination.ListHead))
	{
		return;
	}

	// Set all the parents for the children, then link the head

	TSharedPtr<FViewModel> ListTail;
	if (OutDestination.Owner == Owner)
	{
		// Already same parent so we're just moving from one list to another
		ListTail = GetTail();
	}
	else
	{
		// We purposefully only report changes for the first element
		// Since that is all that is necessary to trigger events for the old parent
		bool bReportChanges = true;

		for (const FViewModelPtr& Child : IterateSubList().ToArray())
		{
			Child->SetParentOnly(OutDestination.Owner, bReportChanges);
			ListTail = Child;

			bReportChanges = false;
		}
	}

	check(ListTail && ListTail->Link.Next == nullptr);

	// Link the destination's current head (and all its subsequent links) to our tail
	TSharedPtr<FViewModel> OldHead = OutDestination.GetHead();
	if (OldHead)
	{
		if (OldHead->SharedData)
		{
			// Trigger a single event for the movement of the old head
			OldHead->SharedData->PreHierarchicalChange(OldHead);
		}

		ListTail->Link.Next = OldHead;
		OldHead->Link.WeakPrev = TSharedPtr<FViewModelListLink>(ListTail, &ListTail->Link);
	}

	TSharedPtr<FViewModel> NewHead = GetHead();

	if (NewHead->SharedData)
	{
		// Trigger a single event for the movement of the head
		NewHead->SharedData->PreHierarchicalChange(NewHead);
	}

	// Reassign the destination head
	OutDestination.ListHead->HeadLink.Next = NewHead;
	NewHead->Link.WeakPrev = TSharedPtr<FViewModelListLink>(Owner, &OutDestination.ListHead->HeadLink);

	// Clear our head
	ListHead->HeadLink.Next = nullptr;
}

void FViewModelChildren::Empty()
{
	if (ListHead->HeadLink.Next)
	{
		FViewModelHierarchyOperation Operation(Owner->GetSharedData());

		for (const FViewModelPtr& Child : IterateSubList().ToArray())
		{
			Child->SetParentOnly(nullptr);
		}

		ListHead->ReliquishList();
	}
}

FViewModelHierarchyOperation::FViewModelHierarchyOperation(const TSharedPtr<FSharedViewModelData>& InSharedData)
	: SharedData(InSharedData)
{
	Construct();
}

void FViewModelHierarchyOperation::Construct()
{
	if (SharedData)
	{
		OldOperation = SharedData->CurrentHierarchicalOperation;
		if (OldOperation)
		{
			AccumulationBuffer = OldOperation->AccumulationBuffer;
		}
		else
		{
			AccumulationBuffer = MakeShared<FOperationAccumulationBuffer>();
		}
		SharedData->CurrentHierarchicalOperation = this;
	}
}

FViewModelHierarchyOperation::~FViewModelHierarchyOperation()
{
	if (SharedData)
	{
		// Reset this before we trigger any events.
		// This ensures that any FViewModelHierarchyOperation's that happen as a result of an event
		// Do not end up restoring this operation on completion (since it's being destroyed!)
		SharedData->CurrentHierarchicalOperation = OldOperation;

		if (OldOperation == nullptr)
		{
			// We were the last one - broadcast events and reset our buffer
			TSet<FViewModel*> AlreadyVisitedModels;

			struct FSortData
			{
				TSharedPtr<FViewModel> Model;
				int32 Depth = 0;
			};
			TArray<FSortData> SortedModels;
			SortedModels.Reserve(AccumulationBuffer->ChangedModels.Num());

			for (const TPair<TWeakPtr<FViewModel>, FCachedHierarchicalPosition>& Pair : AccumulationBuffer->ChangedModels)
			{
				TSharedPtr<FViewModel> Model           = Pair.Key.Pin();
				TSharedPtr<FViewModel> CurrentParent   = Model ? Model->GetParent() : nullptr;
				TSharedPtr<FViewModel> PreviousParent  = Pair.Value.PreviousParent.Pin();
				TSharedPtr<FViewModel> PreviousSibling = Pair.Value.PreviousSibling.Pin();

				// Care is taken to handle a nullptr Model here - this will happen if a model has been removed from a parent and destroyed
				// In such situations we want to ensure that we still trigger an event for the model's parent's if they still exist
				bool bTriggerEvent = Model == nullptr;

				// Only trigger changes if this element was actually moved somewhere different.
				// This can often happen during track row re-generation that ends up re-generating the same layout
				bTriggerEvent = bTriggerEvent || (CurrentParent != PreviousParent || Model->GetPreviousSibling() != PreviousSibling);

				if (bTriggerEvent)
				{
					// We want to ensure that we trigger children first
					if (CurrentParent)
					{
						SortedModels.Add({ CurrentParent, CurrentParent->GetHierarchicalDepth() });
					}
					if (PreviousParent && CurrentParent != PreviousParent)
					{
						SortedModels.Add({ PreviousParent, PreviousParent->GetHierarchicalDepth() });
					}
				}
			}

			if (SortedModels.Num() != 0)
			{
				Algo::SortBy(SortedModels, &FSortData::Depth, TGreater<int32>());

				for (const FSortData& SortElement : SortedModels)
				{
					// Include this since we are always triggering events on a parent model
					constexpr bool bIncludeThis = true;
					for (const FViewModelPtr& Parent : SortElement.Model->GetAncestors(bIncludeThis))
					{
						if (AlreadyVisitedModels.Contains(Parent.Get()))
						{
							break;
						}

						SharedData->BroadcastHierarchicalChange(Parent);
						AlreadyVisitedModels.Add(Parent.Get());
					}
				}
			}

			AccumulationBuffer = nullptr;
		}

		SharedData->PurgeStaleHandlers();
	}
}

void FViewModelHierarchyOperation::PreHierarchicalChange(const TSharedPtr<FViewModel>& InChangedModel)
{
	if (AccumulationBuffer)
	{
		TSharedPtr<FViewModel> Parent = InChangedModel->GetParent();

		if (!AccumulationBuffer->ChangedModels.Contains(InChangedModel))
		{
			AccumulationBuffer->ChangedModels.Add(InChangedModel, { Parent, InChangedModel->GetPreviousSibling() });
		}

		TSharedPtr<FViewModel> NextSibling = InChangedModel->GetNextSibling();
		if (NextSibling && !AccumulationBuffer->ChangedModels.Contains(NextSibling))
		{
			AccumulationBuffer->ChangedModels.Add(NextSibling, { Parent, InChangedModel });
		}
	}
}

} // namespace Sequencer
} // namespace UE

