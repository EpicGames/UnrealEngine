// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE
{
namespace Sequencer
{

FViewModelIterationState::FViewModelIterationState(FViewModelPtr&& InViewModel)
	: ViewModel(MoveTemp(InViewModel))
{
	if (ViewModel)
	{
		++ViewModel->ActiveIterationCount;
	}
}

FViewModelIterationState::FViewModelIterationState(const FViewModelIterationState& RHS)
	: ViewModel(RHS.ViewModel)
{
	if (ViewModel)
	{
		++ViewModel->ActiveIterationCount;
	}
}
FViewModelIterationState& FViewModelIterationState::operator=(const FViewModelIterationState& RHS)
{
	if (ViewModel == RHS.ViewModel)
	{
		return *this;
	}

	// Decrement our existing model's count
	if (ViewModel)
	{
		--ViewModel->ActiveIterationCount;
	}

	// Assign the new model
	ViewModel = RHS.ViewModel;

	// Increment our new model's count
	if (ViewModel)
	{
		++ViewModel->ActiveIterationCount;
	}

	return *this;
}

FViewModelIterationState::FViewModelIterationState(FViewModelIterationState&& RHS)
	: ViewModel(MoveTemp(RHS.ViewModel))
{
	// Reset the old model - thus transferring its active count to this instance if it had one
	RHS.ViewModel = nullptr;
}
FViewModelIterationState& FViewModelIterationState::operator=(FViewModelIterationState&& RHS)
{
	// Decrement our existing count if we have one
	if (ViewModel)
	{
		--ViewModel->ActiveIterationCount;
	}

	ViewModel = MoveTemp(RHS.ViewModel);

	// Reset the old model - thus transferring its active count to this instance if it had one
	RHS.ViewModel = nullptr;

	return *this;
}

FViewModelIterationState::~FViewModelIterationState()
{
	if (ViewModel)
	{
		--ViewModel->ActiveIterationCount;
	}
}


FParentFirstChildIterator::FParentFirstChildIterator(const TSharedPtr<FViewModel>& StartAt, bool bIncludeThis, EViewModelListType InFilter)
	: CurrentItem(StartAt)
	, Filter(InFilter)
	, DepthLimit(-1)
	, bIgnoreCurrentChildren(false)
{
	if (StartAt && !bIncludeThis)
	{
		++*this;
	}
}

void FParentFirstChildIterator::IterateToNext()
{
	// Move the head of the stack to the next potential item
	const bool bCanRecurse = (DepthLimit == -1 || IterStack.Num() < DepthLimit);
	if (bCanRecurse)
	{
		if (IterateToNextChild())
		{
			return;
		}
	}

	IterateToNextSibling();
}

bool FParentFirstChildIterator::IterateToNextChild()
{
	FViewModelListIterator NewIter(CurrentItem->FirstChildListHead, Filter);
	if (NewIter)
	{
		CurrentItem = NewIter.GetCurrentItem();
		IterStack.Add(NewIter);
		return true;
	}
	return false;
}

void FParentFirstChildIterator::IterateToNextSibling()
{
	// Keep popping parents until we find one with a sibling that we haven't visited yet
	// We never reset the root of the 
	for (; IterStack.Num() != 0; )
	{
		FViewModelListIterator& Top = IterStack.Top();
		++Top;
		if (Top)
		{
			CurrentItem = Top.GetCurrentItem();
			break;
		}
		IterStack.Pop();
	}

	if (IterStack.Num() == 0)
	{
		// We've finished iterating now
		CurrentItem = nullptr;
	}
}


} // namespace Sequencer
} // namespace UE
