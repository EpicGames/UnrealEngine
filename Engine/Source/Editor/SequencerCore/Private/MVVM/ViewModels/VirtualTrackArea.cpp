// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/VirtualTrackArea.h"

#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "Misc/FrameRate.h"

namespace UE::Sequencer { class FViewModel; }

namespace UE
{
namespace Sequencer
{

FVirtualTrackArea::FVirtualTrackArea(const FTrackAreaViewModel& InTrackArea, SOutlinerView& InTreeView, const FGeometry& InTrackAreaGeometry)
	: FTimeToPixel(InTrackAreaGeometry, InTrackArea.GetViewRange(), InTrackArea.GetTickResolution())
	, TreeView(InTreeView)
	, TrackAreaGeometry(InTrackAreaGeometry)
{
}

float FVirtualTrackArea::PixelToVerticalOffset(float InPixel) const
{
	return TreeView.PhysicalToVirtual(InPixel);
}

float FVirtualTrackArea::VerticalOffsetToPixel(float InOffset) const
{
	return TreeView.VirtualToPhysical(InOffset);
}

FVector2D FVirtualTrackArea::PhysicalToVirtual(FVector2D InPosition) const
{
	InPosition.Y = PixelToVerticalOffset(InPosition.Y);
	InPosition.X = PixelToSeconds(InPosition.X);

	return InPosition;
}

FVector2D FVirtualTrackArea::VirtualToPhysical(FVector2D InPosition) const
{
	InPosition.Y = VerticalOffsetToPixel(InPosition.Y);
	InPosition.X = SecondsToPixel(InPosition.X);

	return InPosition;
}

FVector2D FVirtualTrackArea::GetPhysicalSize() const
{
	return TrackAreaGeometry.Size;
}

TSharedPtr<UE::Sequencer::FViewModel> FVirtualTrackArea::HitTestNode(float InPhysicalPosition) const
{
	return TreeView.HitTestNode(InPhysicalPosition);
}

} // namespace Sequencer
} // namespace UE
