// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/KeyRenderer.h"
#include "MVVM/Extensions/IHoveredExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/LinkedOutlinerExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

#include "Styling/AppStyle.h"

#include "CurveModel.h"
#include "CurveEditor.h"

#include "Async/ParallelFor.h"
#include "Algo/Accumulate.h"
#include "Containers/Map.h"
#include "Containers/Set.h"

#include "Rendering/DrawElementPayloads.h"

namespace UE::Sequencer
{

void FKeyRenderer::Initialize(const FViewModelPtr& InViewModel)
{
	WeakViewModels = { InViewModel };

	bEvaluatePartialKeysByGroup = false;
	PartialKeyGroupIDsByModelID.Reset();
}

void FKeyRenderer::Initialize(const TSet<FWeakViewModelPtr>& InViewModels)
{
	WeakViewModels = InViewModels;

	bEvaluatePartialKeysByGroup = false;
	PartialKeyGroupIDsByModelID.Reset();
}

void FKeyRenderer::InitializeGrouped(const TSet<FWeakViewModelPtr>& InViewModels, TMap<uint32, uint32> InGroupIDsByModelID)
{
	WeakViewModels = InViewModels;

	bEvaluatePartialKeysByGroup = true;
	PartialKeyGroupIDsByModelID = MoveTemp(InGroupIDsByModelID);
}

bool FKeyRenderer::HasCurve() const
{
	return PrecomputedCurve.Num() >= 2;
}

void FKeyRenderer::HitTestKeys(const FFrameTime& Time, TArray<FKeysForModel>& OutAllKeys, const bool bInLeftAlignCollision) const
{
	const int32 PrecomputedKeysCount = PrecomputedKeys.Num();
	if (PrecomputedKeysCount == 0)
	{
		return;
	}

	const FFrameTime HalfKeyWidth = KeyWidthInFrames * .5f;

	int32 HitIndex = INDEX_NONE;

	if (bInLeftAlignCollision)
	{
		// Left-aligned collision: key occupies [FinalKeyPosition, FinalKeyPosition + KeyWidthInFrames]
		// Keep it fast:
		// - candidates are keys whose start is within one key width behind Time, up to Time
		// - we early-break once Start > Time and we're already getting worse (Start only increases)
		const FFrameTime KeyWidth = KeyWidthInFrames;

		const int32 LowerIndex = Algo::LowerBoundBy(PrecomputedKeys, Time - KeyWidth, &FKey::FinalKeyPosition);
		const int32 UpperIndex = Algo::UpperBoundBy(PrecomputedKeys, Time,            &FKey::FinalKeyPosition);

		if (LowerIndex >= PrecomputedKeys.Num())
		{
			return;
		}

		FFrameTime BestDistance = TNumericLimits<FFrameNumber>::Max();
		for (int32 Index = LowerIndex; Index < UpperIndex && Index < PrecomputedKeys.Num(); ++Index)
		{
			const FKey& Key = PrecomputedKeys[Index];

			const FFrameTime Start = Key.FinalKeyPosition;
			const FFrameTime End   = Start + KeyWidth;

			FFrameTime Distance = FFrameTime(0);

			if (Time < Start)
			{
				Distance = Start - Time;
			}
			else if (Time > End)
			{
				Distance = Time - End;
			}

			if (Distance <= BestDistance)
			{
				BestDistance = Distance;
				HitIndex = Index;

				if (Distance == FFrameTime(0))
				{
					// Can't do better than a direct hit
					break;
				}
			}
			else if (Start > Time)
			{
				// From here on, Start only increases, so Distance = Start-Time only increases.
				break;
			}
		}

		if (HitIndex == INDEX_NONE)
		{
			return;
		}

		const FFrameTime HitStart = PrecomputedKeys[HitIndex].FinalKeyPosition;
		const FFrameTime HitEnd   = HitStart + KeyWidth;
		if (Time < HitStart || Time > HitEnd)
		{
			return;
		}
	}
	else // Center-aligned collision
	{
		// Hit test the actual structure used for drawing keys to ensure we are 100% accurate
		// PrecomputedKeys is an array sorted by time, so we can binary search it to find the hit key
		const int32 LowerIndex = Algo::LowerBoundBy(PrecomputedKeys, Time - HalfKeyWidth, &FKey::FinalKeyPosition);
		const int32 UpperIndex = Algo::UpperBoundBy(PrecomputedKeys, Time + HalfKeyWidth, &FKey::FinalKeyPosition);
		
		if (LowerIndex >= PrecomputedKeysCount)
		{
			// No hit - we ran off the end of the drawn keys
			return;
		}

		// We may have hit many overlapping keys, so we need to choose the one closest to the time
		// Assume the first one is correct, then keep checking the distance between that and the next
		// Break as soon as we find a key that is further away than the last (we know we're moving away from the time at that point)
		HitIndex = LowerIndex;
		for (int32 Index = LowerIndex + 1; Index < UpperIndex && Index < PrecomputedKeysCount; ++Index)
		{
			const FKey& PrevKey = PrecomputedKeys[Index - 1];
			const FKey& ThisKey = PrecomputedKeys[Index];

			// Find the distance from the middle of the key to the hit time
			FFrameTime PrevKeyDistance = FMath::Abs((PrevKey.KeyTickStart + PrevKey.KeyTickEnd) * .5f - Time);
			FFrameTime ThisKeyDistance = FMath::Abs((ThisKey.KeyTickStart + ThisKey.KeyTickEnd) * .5f - Time);

			if (ThisKeyDistance <= PrevKeyDistance)
			{
				HitIndex = Index;
			}
			else
			{
				break;
			}
		}

		if (!(HitIndex < PrecomputedKeys.Num()
			&& FMath::Abs(PrecomputedKeys[HitIndex].FinalKeyPosition - Time) <= HalfKeyWidth))
		{
			return;
		}
	}

	const FFrameTime Start = PrecomputedKeys[HitIndex].KeyTickStart;
	const FFrameTime End   = PrecomputedKeys[HitIndex].KeyTickEnd;

	// We have a hit. Gather anything from the key draw info that overlaps the hit range
	for (const FCachedKeyDrawInformation& Entry : KeyDrawInfo)
	{
		const int32 FrameStart = Algo::LowerBound(Entry.FramesInRange, Start);
		const int32 FrameEnd   = Algo::UpperBound(Entry.FramesInRange, End);
		const int32 NumKeys    = FrameEnd - FrameStart;

		if (NumKeys > 0 && FrameStart < Entry.HandlesInRange.Num())
		{
			FKeysForModel Keys;
			Keys.Model = Entry.WeakKeyExtension.Pin();
			Keys.Keys.SetNumUninitialized(NumKeys);

			for (int32 Index = 0; Index < NumKeys; ++Index)
			{
				Keys.Keys[Index] = Entry.HandlesInRange[FrameStart + Index];
			}

			OutAllKeys.Emplace(MoveTemp(Keys));
		}
	}
}

bool FKeyRenderer::HitTestKeyBar(const FFrameTime& Time, FKeyBar& OutKeyBar) const
{
	const FFrameTime HalfKeyWidth = KeyWidthInFrames * .5f;

	// Hit test the actual structure used for drawing key bars to ensure we are 100% accurate
	// PrecomputedKeyBars is an array sorted by time, so we can binary search it to find the hit area
	const int32 HitIndex = Algo::LowerBoundBy(PrecomputedKeyBars, Time, &FCachedKeyBar::EndTime);
	if (HitIndex >= PrecomputedKeyBars.Num() || PrecomputedKeyBars[HitIndex].StartTime > Time)
	{
		// No hit - we ran off the end of the drawn keys, or didn't hit a bar
		return false;
	}

 	OutKeyBar.Range = TRange<FFrameTime>::Inclusive(PrecomputedKeyBars[HitIndex].StartTime+HalfKeyWidth, PrecomputedKeyBars[HitIndex].EndTime-HalfKeyWidth);

	// Generate leading keys array
	{
		FFrameTime LeadingStart = PrecomputedKeyBars[HitIndex].StartTime - HalfKeyWidth;
		FFrameTime LeadingEnd   = PrecomputedKeyBars[HitIndex].StartTime + HalfKeyWidth;

		for (const FCachedKeyDrawInformation& Entry : KeyDrawInfo)
		{
			const int32 FrameStart = Algo::LowerBound(Entry.CachedKeys->KeyTimes, LeadingStart);
			const int32 FrameEnd   = Algo::UpperBound(Entry.CachedKeys->KeyTimes, LeadingEnd);
			const int32 NumKeys    = FrameEnd - FrameStart;

			if (NumKeys > 0 && FrameStart < Entry.CachedKeys->KeyHandles.Num())
			{
				FKeysForModel Keys;
				Keys.Model = Entry.WeakKeyExtension.Pin();
				Keys.Keys.SetNumUninitialized(NumKeys);

				for (int32 Index = 0; Index < NumKeys; ++Index)
				{
					Keys.Keys[Index] = Entry.CachedKeys->KeyHandles[FrameStart + Index];
				}

				OutKeyBar.LeadingKeys.Emplace(MoveTemp(Keys));
			}
		}
	}

	// Generate trailing keys array
	{
		FFrameTime TrailingStart = PrecomputedKeyBars[HitIndex].EndTime - HalfKeyWidth;
		FFrameTime TrailingEnd   = PrecomputedKeyBars[HitIndex].EndTime + HalfKeyWidth;

		for (const FCachedKeyDrawInformation& Entry : KeyDrawInfo)
		{
			const int32 FrameStart = Algo::LowerBound(Entry.CachedKeys->KeyTimes, TrailingStart);
			const int32 FrameEnd   = Algo::UpperBound(Entry.CachedKeys->KeyTimes, TrailingEnd);
			const int32 NumKeys    = FrameEnd - FrameStart;

			if (NumKeys > 0 && FrameStart < Entry.CachedKeys->KeyHandles.Num())
			{
				FKeysForModel Keys;
				Keys.Model = Entry.WeakKeyExtension.Pin();
				Keys.Keys.SetNumUninitialized(NumKeys);

				for (int32 Index = 0; Index < NumKeys; ++Index)
				{
					Keys.Keys[Index] = Entry.CachedKeys->KeyHandles[FrameStart + Index];
				}

				OutKeyBar.TrailingKeys.Emplace(MoveTemp(Keys));
			}
		}
	}

	return true;
}

void FKeyRenderer::Update(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry) const
{
	if (EnumHasAnyFlags(Params.CacheState, EViewDependentCacheFlags::Empty))
	{
		KeyDrawInfo.Reset();
		return;
	}

	if (EnumHasAnyFlags(Params.CacheState, EViewDependentCacheFlags::DataChanged))
	{
		CacheKeyExtensions(Params);
	}

	FKeyBatchParameters NewParams = Params;
	if (Params.bUpdateViewIndependentData)
	{
		NewParams.CacheState |= UpdateViewIndependentData();
	}

	// We can reuse this key layout - update all the cached key positions
	UpdateViewDependentData(NewParams, AllottedGeometry);
}

EViewDependentCacheFlags FKeyRenderer::UpdateViewIndependentData() const
{
	EViewDependentCacheFlags CacheFlags = EViewDependentCacheFlags::None;

	for (FCachedKeyDrawInformation& CachedKeyDrawInfo : KeyDrawInfo)
	{
		CacheFlags |= CachedKeyDrawInfo.UpdateViewIndependentData();
	}

	return CacheFlags;
}

void FKeyRenderer::CacheKeyExtensions(const FKeyBatchParameters& Params) const
{
	KeyDrawInfo.Empty();

	// Walk through the current models and accumulate key renderer extensions for any collapsed children
	TMap<uint32, int32> GroupIndices;

	const auto GetGroupIndex = [this, &GroupIndices](const FViewModelPtr& CurrentViewModel) -> int32
	{
		const uint32 ModelID = CurrentViewModel->GetModelID();
		const uint32* const PartialKeyGroupID = PartialKeyGroupIDsByModelID.Find(ModelID);
		const uint32 GroupID = PartialKeyGroupID ? *PartialKeyGroupID : ModelID;

		if (int32* const ExistingGroupIndex = GroupIndices.Find(GroupID))
		{
			return *ExistingGroupIndex;
		}

		const int32 NewGroupIndex = GroupIndices.Num();
		GroupIndices.Add(GroupID, NewGroupIndex);
		return NewGroupIndex;
	};

	for (const FWeakViewModelPtr& WeakViewModel : WeakViewModels)
	{
		const TViewModelPtr<FViewModel> ViewModel = WeakViewModel.Pin();
		if (!ViewModel.IsValid())
		{
			continue;
		}

		constexpr bool bIncludeThis = true;
		for (FParentFirstChildIterator ChildIt(ViewModel, bIncludeThis) ; ChildIt; ++ChildIt)
		{
			FViewModelPtr CurrentViewModel = *ChildIt;

			TViewModelPtr<FLinkedOutlinerExtension> OutlinerExtension = CurrentViewModel.ImplicitCast();
			if (OutlinerExtension && OutlinerExtension->GetLinkedOutlinerItem() && OutlinerExtension->GetLinkedOutlinerItem()->IsFilteredOut())
			{
				ChildIt.IgnoreCurrentChildren();
				continue;
			}

			if (const TViewModelPtr<IKeyExtension> KeyExtension = CurrentViewModel.ImplicitCast())
			{
				const int32 GroupIndex = bEvaluatePartialKeysByGroup ? GetGroupIndex(CurrentViewModel) : 0;
				KeyDrawInfo.Emplace(KeyExtension, GroupIndex);
			}

			if (Params.bCollapseChildren == false)
			{
				break;
			}
		}
	}
}

void FKeyRenderer::UpdateViewDependentData(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry) const
{
	if (Params.CacheState == EViewDependentCacheFlags::None)
	{
		// Cache is still hot - nothing to do
		return;
	}

	KeyWidthInFrames = Params.TimeToPixel.PixelDeltaToFrame(Params.KeySizePx.X);

	const bool bHasAnySelection        = Params.ClientInterface && Params.ClientInterface->HasAnySelectedKeys();
	const bool bHasAnySelectionPreview = Params.ClientInterface && Params.ClientInterface->HasAnyPreviewSelectedKeys();
	const bool bHasAnyHoveredKeys      = Params.ClientInterface && Params.ClientInterface->HasAnyHoveredKeys();

	const int32 NumFramesInRange = Algo::Accumulate(KeyDrawInfo, 0, [](int32 Current, const FCachedKeyDrawInformation& In) { return Current + In.Num(); });

	// ------------------------------------------------------------------------------
	// Update view-dependent data for each draw info
	const bool bForceSingleThread = NumFramesInRange <= 50000;
	ParallelFor(KeyDrawInfo.Num(), [this, &Params](int32 Index){
		this->KeyDrawInfo[Index].CacheViewDependentData(Params);
	}, bForceSingleThread);

	// ------------------------------------------------------------------------------
	// If the data has changed, or key state has changed, or the view has been zoomed
	// we cannot preserve any keys (because we don't know whether they are still valid)
	const bool bCanPreserveKeys = !EnumHasAnyFlags(Params.CacheState, EViewDependentCacheFlags::DataChanged | EViewDependentCacheFlags::ViewZoomed | EViewDependentCacheFlags::KeyStateChanged);

	double PreserveStartFrame = TNumericLimits<double>::Max();
	TArray<FKey> PreservedKeys;

	// Attempt to preserve any previously computed key draw information
	if (bCanPreserveKeys && PrecomputedKeys.Num() != 0)
	{
		const FFrameTime LowerBoundFrame = Params.VisibleRange.GetLowerBoundValue();
		const FFrameTime UpperBoundFrame = Params.VisibleRange.GetUpperBoundValue();

		const int32 PreserveStartIndex = Algo::LowerBoundBy(PrecomputedKeys, LowerBoundFrame, &FKey::KeyTickStart);
		const int32 PreserveEndIndex   = Algo::UpperBoundBy(PrecomputedKeys, UpperBoundFrame, &FKey::KeyTickEnd);

		const int32 PreserveNum = PreserveEndIndex - PreserveStartIndex;
		if (PreserveNum > 0)
		{
			PreservedKeys = TArray<FKey>(PrecomputedKeys.GetData() + PreserveStartIndex, PreserveNum);
			PreserveStartFrame = PreservedKeys[0].KeyTickStart.AsDecimal();

			FFrameTime ActualPreserveEndFrame = PreservedKeys.Last().KeyTickEnd;
			for (FCachedKeyDrawInformation& Info : KeyDrawInfo)
			{
				Info.PreserveToIndex = Algo::UpperBound(Info.FramesInRange, ActualPreserveEndFrame);
			}
		}
	}

	// ------------------------------------------------------------------------------
	// Begin precomputation of keys to draw
	PrecomputedKeys.Reset();

	static float PixelOverlapThreshold = 3.f;
	const double TimeOverlapThreshold = Params.TimeToPixel.PixelDeltaToFrame(PixelOverlapThreshold).AsDecimal();

	auto AnythingLeftToDraw = [](const FCachedKeyDrawInformation& In)
	{
		return In.NextUnhandledIndex < In.FramesInRange.Num();
	};

	// Keep iterating all the cached key positions until we've moved through everything
	// As stated above - this loop does not scale well for large numbers of KeyDrawInfo
	// Which is generally not a problem, but is troublesome for Control Rigs
	while (KeyDrawInfo.ContainsByPredicate(AnythingLeftToDraw))
	{
		// Determine the next key position to draw
		double CardinalKeyFrame = TNumericLimits<double>::Max();
		for (const FCachedKeyDrawInformation& Info : KeyDrawInfo)
		{
			if (Info.NextUnhandledIndex < Info.FramesInRange.Num())
			{
				CardinalKeyFrame = FMath::Min(CardinalKeyFrame, Info.FramesInRange[Info.NextUnhandledIndex].AsDecimal());
			}
		}

		// If the cardinal time overlaps the preserved range, skip those keys
		if (CardinalKeyFrame >= PreserveStartFrame && PreservedKeys.Num() != 0)
		{
			PrecomputedKeys.Append(PreservedKeys);
			PreservedKeys.Empty();
			for (FCachedKeyDrawInformation& Info : KeyDrawInfo)
			{
				Info.NextUnhandledIndex = Info.PreserveToIndex;
			}
			continue;
		}

		// Start grouping keys at the current key time plus 99% of the threshold to ensure that we group at the center of keys
		// and that we avoid floating point precision issues where there is only one key [(KeyTime + TimeOverlapThreshold) - KeyTime != TimeOverlapThreshold] for some floats
		double CardinalKeyTime = CardinalKeyFrame + TimeOverlapThreshold*0.9994f;

		TSet<int32> GroupsWithKeyAtCardinalTime;

		if (bEvaluatePartialKeysByGroup)
		{
			for (const FCachedKeyDrawInformation& Info : KeyDrawInfo)
			{
				if (Info.NextUnhandledIndex < Info.FramesInRange.Num()
					&& FMath::IsNearlyEqual(Info.FramesInRange[Info.NextUnhandledIndex].AsDecimal(), CardinalKeyTime, TimeOverlapThreshold))
				{
					GroupsWithKeyAtCardinalTime.Add(Info.GroupIndex);
				}
			}
		}

		// Track whether all of the keys are within the valid range
		bool bIsInRange = true;
		// Track whether a key is on a whole frame or a sub frame
		bool bIsOnSubFrame = false;

		const FFrameTime ValidPlayRangeMin = Params.ValidPlayRangeMin;
		const FFrameTime ValidPlayRangeMax = Params.ValidPlayRangeMax;

		double AverageKeyTime = 0.0;
		int32 NumKeyTimes = 0;

		FFrameTime KeyTickStart = TNumericLimits<FFrameNumber>::Max();
		FFrameTime KeyTickEnd   = TNumericLimits<FFrameNumber>::Lowest();
		EKeyConnectionStyle ConnectionStyle = EKeyConnectionStyle::None;

		auto HandleKey = [&bIsInRange, &bIsOnSubFrame, &AverageKeyTime, &NumKeyTimes, &KeyTickStart, &KeyTickEnd
			, ValidPlayRangeMin, ValidPlayRangeMax]
		(const FFrameTime& KeyFrame)
		{
			if (bIsInRange && (KeyFrame < ValidPlayRangeMin || KeyFrame >= ValidPlayRangeMax))
			{
				bIsInRange = false;
			}

			bIsOnSubFrame = KeyFrame.GetSubFrame() != 0.f;

			KeyTickStart = FMath::Min(KeyFrame, KeyTickStart);
			KeyTickEnd   = FMath::Max(KeyFrame, KeyTickEnd);

			AverageKeyTime += KeyFrame.AsDecimal();
			++NumKeyTimes;
		};


		bool bFoundKey = false;
		FKey NewKey;

		int32 NumPreviewSelected = 0;
		int32 NumPreviewNotSelected = 0;
		int32 NumSelected = 0;
		int32 NumHovered = 0;
		int32 TotalNumKeys = 0;
		int32 NumOverlaps = 0;
		TMap<int32, FKeyDrawParams> GroupDrawParams;

		static int32 MaxNumKeysToConsider = 500;

		// Determine the ranges of keys considered to reside at this position
		for (int32 DrawIndex = 0; DrawIndex < KeyDrawInfo.Num(); ++DrawIndex)
		{
			FCachedKeyDrawInformation& Info = KeyDrawInfo[DrawIndex];
			TViewModelPtr<IKeyExtension> KeyExtension = Info.WeakKeyExtension.Pin();
			if (!KeyExtension)
			{
				continue;
			}
			if (Info.NextUnhandledIndex >= Info.FramesInRange.Num())
			{
				if (!bEvaluatePartialKeysByGroup || GroupsWithKeyAtCardinalTime.Contains(Info.GroupIndex))
				{
					NewKey.Flags |= EKeyRenderingFlags::PartialKey;
				}
				continue;
			}
			else if (!FMath::IsNearlyEqual(Info.FramesInRange[Info.NextUnhandledIndex].AsDecimal(), CardinalKeyTime, TimeOverlapThreshold))
			{
				if (!bEvaluatePartialKeysByGroup || GroupsWithKeyAtCardinalTime.Contains(Info.GroupIndex))
				{
					NewKey.Flags |= EKeyRenderingFlags::PartialKey;
				}
				continue;
			}

			int32 ThisNumOverlaps = -1;
			do
			{
				HandleKey(Info.FramesInRange[Info.NextUnhandledIndex]);
				const FKeyDrawParams& CurrentDrawParams = Info.DrawParams[Info.NextUnhandledIndex];

				if (!bFoundKey)
				{
					NewKey.Params = CurrentDrawParams;
					bFoundKey = true;
				}
				else if (!bEvaluatePartialKeysByGroup && CurrentDrawParams != NewKey.Params)
				{
					NewKey.Flags |= EKeyRenderingFlags::PartialKey;
				}

				if (bEvaluatePartialKeysByGroup)
				{
					if (const FKeyDrawParams* const ExistingGroupDrawParams = GroupDrawParams.Find(Info.GroupIndex))
					{
						if (CurrentDrawParams != *ExistingGroupDrawParams)
						{
							NewKey.Flags |= EKeyRenderingFlags::PartialKey;
						}
					}
					else
					{
						GroupDrawParams.Add(Info.GroupIndex, CurrentDrawParams);
					}
				}

				NewKey.Params.ConnectionStyle |= CurrentDrawParams.ConnectionStyle;

				// Avoid creating FSequencerSelectedKeys unless absolutely necessary
				FKeyHandle ThisKeyHandle = Info.HandlesInRange[Info.NextUnhandledIndex];

				if (bHasAnySelection && NumSelected < MaxNumKeysToConsider && Params.ClientInterface->IsKeySelected(KeyExtension, ThisKeyHandle))
				{
					++NumSelected;
				}
				if (bHasAnySelectionPreview && NumPreviewSelected < MaxNumKeysToConsider && NumPreviewNotSelected < MaxNumKeysToConsider)
				{
					EKeySelectionPreviewState SelectionState = Params.ClientInterface->GetPreviewSelectionState(KeyExtension, ThisKeyHandle);

					NumPreviewSelected    += int32(SelectionState == EKeySelectionPreviewState::Selected);
					NumPreviewNotSelected += int32(SelectionState == EKeySelectionPreviewState::NotSelected);
				}
				if (bHasAnyHoveredKeys && NumHovered < MaxNumKeysToConsider && Params.ClientInterface->IsKeyHovered(KeyExtension, ThisKeyHandle))
				{
					++NumHovered;
				}

				++TotalNumKeys;
				++Info.NextUnhandledIndex;
				++ThisNumOverlaps;
			}
			while (Info.NextUnhandledIndex < Info.FramesInRange.Num() && FMath::IsNearlyEqual(Info.FramesInRange[Info.NextUnhandledIndex].AsDecimal(), CardinalKeyTime, TimeOverlapThreshold));

			NumOverlaps += ThisNumOverlaps;
		}

		if (NumKeyTimes == 0)
		{
			// This is not actually possible since HandleKey must have been called
			// at least once, but it needs to be here to avoid a static analysis warning
			break;
		}

		NewKey.FinalKeyPosition = FFrameTime::FromDecimal(AverageKeyTime / NumKeyTimes);
		NewKey.KeyTickStart = KeyTickStart;
		NewKey.KeyTickEnd = KeyTickEnd;

		if (EnumHasAnyFlags(NewKey.Flags, EKeyRenderingFlags::PartialKey))
		{
			static const FSlateBrush* PartialKeyBrush = FAppStyle::GetBrush("Sequencer.PartialKey");
			NewKey.Params.FillBrush = NewKey.Params.BorderBrush = PartialKeyBrush;
		}

		// Determine the key color based on its selection/hover states
		if (NumPreviewSelected == FMath::Min(TotalNumKeys, MaxNumKeysToConsider))
		{
			NewKey.Flags |= EKeyRenderingFlags::PreviewSelected;
		}
		else if (NumPreviewNotSelected == FMath::Min(TotalNumKeys, MaxNumKeysToConsider))
		{
			NewKey.Flags |= EKeyRenderingFlags::PreviewNotSelected;
		}
		else if (NumSelected == FMath::Min(TotalNumKeys, MaxNumKeysToConsider))
		{
			NewKey.Flags |= EKeyRenderingFlags::Selected;
		}
		else if (NumSelected != 0)
		{
			NewKey.Flags |= EKeyRenderingFlags::AnySelected;
		}
		else if (NumHovered == FMath::Min(TotalNumKeys, MaxNumKeysToConsider))
		{
			NewKey.Flags |= EKeyRenderingFlags::Hovered;
		}

		if (NumOverlaps > 0)
		{
			NewKey.Flags |= EKeyRenderingFlags::Overlaps;
		}

		if (!bIsInRange)
		{
			NewKey.Flags |= EKeyRenderingFlags::OutOfRange;
		}

		if (bIsOnSubFrame)
		{
			NewKey.Flags |= EKeyRenderingFlags::SubFrame;
		}

		const int32 NumStyles = FMath::CountBits(static_cast<__underlying_type(EKeyConnectionStyle)>(NewKey.Params.ConnectionStyle));
		if (NumStyles > 1)
		{
			NewKey.Params.ConnectionStyle = EKeyConnectionStyle::Dotted;
		}

		PrecomputedKeys.Add(NewKey);
	}

	// Now add connecting bars for the keys

	PrecomputedKeyBars.Reset();
	if (Params.bShowKeyBars)
	{
		const FKeyDrawParams* LeadingKeyParams = nullptr;
		const FKeyDrawParams* TrailingKeyParams = nullptr;

		const FCachedKeys::FCachedKey* LeadingKey = nullptr;
		const FCachedKeys::FCachedKey* TrailingKey = nullptr;

		for (const FCachedKeyDrawInformation& Info : KeyDrawInfo)
		{
			if (Info.LeadingKey.IsValid())
			{
				if (!LeadingKey || Info.LeadingKey.Time > LeadingKey->Time)
				{
					LeadingKey = &Info.LeadingKey;
					LeadingKeyParams = &Info.LeadingKeyParams;
				}
			}
			if (Info.TrailingKey.IsValid())
			{
				if (!TrailingKey || Info.TrailingKey.Time < TrailingKey->Time)
				{
					TrailingKey = &Info.TrailingKey;
					TrailingKeyParams = &Info.TrailingKeyParams;
				}
			}
		}

		if (PrecomputedKeys.Num() == 0)
		{
			if (LeadingKey && TrailingKey)
			{
				PrecomputedKeyBars.Add(FCachedKeyBar{ LeadingKey->Time, TrailingKey->Time, EKeyBarRenderingFlags::None });
			}
		}
		else
		{
			if (LeadingKey && LeadingKeyParams->ConnectionStyle != EKeyConnectionStyle::None &&
				Params.TimeToPixel.FrameDeltaToPixel(PrecomputedKeys[0].FinalKeyPosition - LeadingKey->Time) > Params.KeySizePx.X)
			{
				PrecomputedKeyBars.Add(FCachedKeyBar{ LeadingKey->Time, PrecomputedKeys[0].FinalKeyPosition, EKeyBarRenderingFlags::None, LeadingKeyParams->ConnectionStyle });
			}

			for (int32 Index = 0; Index < PrecomputedKeys.Num()-1; ++Index)
			{
				FKey ThisKey = PrecomputedKeys[Index];
				FKey NextKey = PrecomputedKeys[Index+1];
				if (ThisKey.Params.ConnectionStyle != EKeyConnectionStyle::None &&
					Params.TimeToPixel.FrameDeltaToPixel(NextKey.FinalKeyPosition - ThisKey.FinalKeyPosition) > Params.KeySizePx.X)
				{
					PrecomputedKeyBars.Add(FCachedKeyBar{ ThisKey.FinalKeyPosition, NextKey.FinalKeyPosition, EKeyBarRenderingFlags::None, ThisKey.Params.ConnectionStyle });
				}
			}

			if (TrailingKey && PrecomputedKeys.Last().Params.ConnectionStyle != EKeyConnectionStyle::None &&
				Params.TimeToPixel.FrameDeltaToPixel(TrailingKey->Time - PrecomputedKeys.Last().FinalKeyPosition) > Params.KeySizePx.X)
			{
				PrecomputedKeyBars.Add(FCachedKeyBar{ PrecomputedKeys.Last().FinalKeyPosition, TrailingKey->Time, EKeyBarRenderingFlags::None, PrecomputedKeys.Last().Params.ConnectionStyle });
			}
		}
	}

	PrecomputedCurve.Reset();
	if (Params.bShowCurve)
	{
		for (FCachedKeyDrawInformation& Info : KeyDrawInfo)
		{
			TViewModelPtr<IKeyExtension> KeyExtension = Info.WeakKeyExtension.Pin();
			if (!KeyExtension)
			{
				continue;
			}

			TUniquePtr<FCurveModel> CurveModel = KeyExtension->CreateCurveModel();
			if (CurveModel)
			{
				// Curve editor expects things in seconds
				const double MinTimeSeconds = Params.VisibleRange.GetLowerBoundValue() / Params.TimeToPixel.GetTickResolution();
				const double MaxTimeSeconds = Params.VisibleRange.GetUpperBoundValue() / Params.TimeToPixel.GetTickResolution();
				const double TimeRangeSeconds = MaxTimeSeconds - MinTimeSeconds;

				double Min = 0.0;
				double Max = 1.0;
				if (!KeyExtension->GetFixedExtents(Min, Max))
				{
					CurveModel->GetValueRange(Min, Max);
				}

				if (Min == Max)
				{
					PrecomputedCurve.Add(MakeTuple(Params.TimeToPixel.FrameToPixel(Params.VisibleRange.GetLowerBoundValue()), 0.0f));
					PrecomputedCurve.Add(MakeTuple(Params.TimeToPixel.FrameToPixel(Params.VisibleRange.GetUpperBoundValue()), 0.0f));
				}
				else
				{
					const double Range = Max - Min;

					FCurveEditor Dummy;

					// Set up a screen space that encompasses the entire curve, for the duration of the visible range
					FCurveEditorScreenSpace ScreenSpace(
						AllottedGeometry.GetLocalSize(),
						MinTimeSeconds, MaxTimeSeconds,
						Min, Max);

					CurveModel->DrawCurve(Dummy, ScreenSpace, PrecomputedCurve);

					// Put the points in pixel space, and normalize the values
					for (TTuple<double, double>& Pair : PrecomputedCurve)
					{
						Pair.Key = Params.TimeToPixel.SecondsToPixel(Pair.Key);
						if (Range > 0.0)
						{
							Pair.Value = (Pair.Value - Min) / Range;
						}
					}
				}
			}
		}
	}
}

int32 FKeyRenderer::DrawCurve(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, const FKeyRendererPaintArgs& PaintArgs, int32 LayerId) const
{
	PaintArgs.DrawElements->PushClip(FSlateClippingZone(AllottedGeometry));

	TArray<FVector2D> CurvePoints;
	for (const TTuple<double, double>& CurveKey : PrecomputedCurve)
	{
		CurvePoints.Add(
			FVector2D(
				CurveKey.Get<0>(),
				(1.0f - CurveKey.Get<1>()) * (AllottedGeometry.GetLocalSize().Y-2.f) + 1.f // Inset the curve 1 px so it doesn't clip
			)
		);
	}

	const float CurveThickness = 1.f;
	const bool  bAntiAliasCurves = true;

	FSlateDrawElement::MakeLines(
		*PaintArgs.DrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		CurvePoints,
		PaintArgs.DrawEffects,
		PaintArgs.CurveColor,
		bAntiAliasCurves,
		CurveThickness
	);

	PaintArgs.DrawElements->PopClip();
	return LayerId + 1;
}

int32 FKeyRenderer::Draw(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, const FKeyRendererPaintArgs& PaintArgs, int32 LayerId) const
{
	const FVector2f LocalSize = AllottedGeometry.GetLocalSize();
	const FVector2f HalfLocalSize = LocalSize * .5f;

	// Draw key bars first
	{
		static const FSlateBrush* DottedKeyBarBrush = FAppStyle::GetBrush("Sequencer.KeyBar.Dotted");
		static const FSlateBrush* DashedKeyBarBrush = FAppStyle::GetBrush("Sequencer.KeyBar.Dashed");
		static const FSlateBrush* SolidKeyBarBrush = FAppStyle::GetBrush("Sequencer.KeyBar.Solid");

		const float KeyBarFadeWidthPx = MyCullingRect.GetSize2f().X;
		const float KeyBarFadeThresholdPx = KeyBarFadeWidthPx * 0.25f;

		static float KeyBarHeight = 2.f;
		const float KeyBarTop = HalfLocalSize.Y - KeyBarHeight*.5f;
		for (const FCachedKeyBar& KeyBar : PrecomputedKeyBars)
		{
			const float StartPx = Params.TimeToPixel.FrameToPixel(KeyBar.StartTime);
			const float EndPx = Params.TimeToPixel.FrameToPixel(KeyBar.EndTime);
			const float KeyBarWidth = EndPx - StartPx;

			const float FadeAlpha = 1.f - FMath::Clamp((KeyBarWidth - KeyBarFadeThresholdPx) / (KeyBarFadeWidthPx - KeyBarFadeThresholdPx), 0.f, 1.f);
			const float Opacity = FMath::Lerp(.25f, 1.f, FadeAlpha);

			const FSlateBrush* Brush = SolidKeyBarBrush;
			if (EnumHasAnyFlags(KeyBar.ConnectionStyle, EKeyConnectionStyle::Dotted))
			{
				Brush = DottedKeyBarBrush;
			}
			else if (EnumHasAnyFlags(KeyBar.ConnectionStyle, EKeyConnectionStyle::Dashed))
			{
				Brush = DashedKeyBarBrush;
			}

			if (Opacity < 1.f && Brush == SolidKeyBarBrush)
			{
				TArray<FSlateGradientStop> Stops;
				Stops.Add(FSlateGradientStop(FVector2d(0.0, 0.0), PaintArgs.KeyBarColor));
				Stops.Add(FSlateGradientStop(FVector2d(KeyBarFadeThresholdPx*0.5f, 0.0), PaintArgs.KeyBarColor.CopyWithNewOpacity(Opacity)));
				Stops.Add(FSlateGradientStop(FVector2d(KeyBarWidth - KeyBarFadeThresholdPx*0.5f * 0.75f, 0.0), PaintArgs.KeyBarColor.CopyWithNewOpacity(Opacity)));
				Stops.Add(FSlateGradientStop(FVector2d(KeyBarWidth, 0.0), PaintArgs.KeyBarColor));

				FSlateDrawElement::MakeGradient(
					*PaintArgs.DrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(
						FVector2D(KeyBarWidth, KeyBarHeight),
						FSlateLayoutTransform(FVector2D(StartPx, KeyBarTop))
					),
					MoveTemp(Stops),
					EOrientation::Orient_Vertical,
					PaintArgs.DrawEffects
				);
			}
			else
			{
				FSlateDrawElement::MakeBox(
					*PaintArgs.DrawElements,
					LayerId,
					// Center the key along Y.  Ensure the middle of the key is at the actual key time
					AllottedGeometry.ToPaintGeometry(
						FVector2D(EndPx - StartPx, KeyBarHeight),
						FSlateLayoutTransform(FVector2D(StartPx, KeyBarTop))
					),
					Brush,
					PaintArgs.DrawEffects,
					PaintArgs.KeyBarColor.CopyWithNewOpacity(Opacity)
				);
			}
		}
		++LayerId;
	}

	static constexpr FLinearColor KeyBorderTint = FLinearColor(0.05f, 0.05f, 0.05f, 1.0f);
	static constexpr FLinearColor KeyOverlapTint = FLinearColor(0.83f, 0.12f, 0.12f, 1.0f); // Red

	for (const FKey& Key : PrecomputedKeys)
	{
		FKeyDrawParams KeyDrawParams = Key.Params;

		KeyDrawParams.FillTint = PaintArgs.KeyFillColor;
		KeyDrawParams.BorderTint = KeyBorderTint;

		const bool bPartialKey = EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::PartialKey);
		const bool bSelected = EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::Selected);
		const bool bPartiallySelected = EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::AnySelected);
		const bool bHovered = EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::Hovered);
		const bool bSubFrame = EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::SubFrame);
		const bool bOverlapped = EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::Overlaps);

		if (bPartialKey)
		{
			KeyDrawParams.FillOffset = FVector2D(0.f, 0.f);
			KeyDrawParams.FillTint = PaintArgs.PartialKeyFillColor;
			KeyDrawParams.BorderTint = PaintArgs.PartialKeyFillColor;
		}		

		// Determine the key color based on its selection/hover states
		if (EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::PreviewSelected))
		{
			FLinearColor PreviewSelectionColor = PaintArgs.KeySelectionFillColor.LinearRGBToHSV();
			PreviewSelectionColor.R += 0.1f; // +10% hue
			PreviewSelectionColor.G = 0.6f; // 60% saturation
			KeyDrawParams.BorderTint = KeyDrawParams.FillTint = PreviewSelectionColor.HSVToLinearRGB();
		}
		else if (EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::PreviewNotSelected))
		{
			KeyDrawParams.BorderTint = KeyBorderTint;
		}
		else if (bSelected)
		{
			KeyDrawParams.BorderTint = PaintArgs.KeySelectionFillColor;
			KeyDrawParams.FillTint = PaintArgs.KeySelectionFillColor;
		}
		else if (bPartiallySelected)
		{
			KeyDrawParams.BorderTint = PaintArgs.PartialKeySelectionFillColor;
			KeyDrawParams.FillTint =  PaintArgs.PartialKeySelectionFillColor;
		}
		else if (bHovered)
		{
			FLinearColor HoveredKeyFillColor = PaintArgs.HoveredKeyFillColor;

			if (PaintArgs.bBlendHoveredKeyColorWithBaseKeyColor)
			{
				const float ClampedBlendFactor = FMath::Clamp(PaintArgs.HoveredKeyBlendFactor, 0.f, 1.f);
				HoveredKeyFillColor = FLinearColor::LerpUsingHSV(KeyDrawParams.FillTint, PaintArgs.HoveredKeyFillColor, ClampedBlendFactor);
			}

			KeyDrawParams.BorderTint = HoveredKeyFillColor;
			KeyDrawParams.FillTint = HoveredKeyFillColor;
		}

		// Color keys with overlaps with a red border
		if (bOverlapped)
		{
			KeyDrawParams.BorderTint = KeyOverlapTint;
		}

		const ESlateDrawEffect KeyDrawEffects = EnumHasAnyFlags(Key.Flags, EKeyRenderingFlags::OutOfRange)
			? ESlateDrawEffect::DisabledEffect : PaintArgs.DrawEffects;

		// draw border
		const float KeyPositionPx = Params.TimeToPixel.FrameToPixel(Key.FinalKeyPosition);

		FVector2D KeySize = bSelected
			? Params.KeySizePx + (PaintArgs.ThrobAmount * PaintArgs.KeyThrobValue)
			: Params.KeySizePx;
		KeySize.X = FMath::Max(KeySize.X, 1.f);
		const FVector2D HalfKeySize = KeySize * .5f;

		if (PaintArgs.KeyStyle == EKeyRendererStyle::Line
			|| PaintArgs.KeyStyle == EKeyRendererStyle::FrameBlock)
		{
			const FPaintGeometry KeyFrameGeometry = AllottedGeometry.ToPaintGeometry(
				KeySize,
				FSlateLayoutTransform(FVector2D(KeyPositionPx, 0.) + PaintArgs.KeyOffset)
			);

			KeyDrawParams.FillBrush = bSubFrame ? FAppStyle::GetBrush(TEXT("Checker"))
				: FAppStyle::GetBrush(TEXT("Brushes.White"));
			KeyDrawParams.BorderBrush = FAppStyle::GetBrush(TEXT("PlainBorder"));

			// Draw fill
			FSlateDrawElement::MakeBox(
				*PaintArgs.DrawElements,
				bSelected ? LayerId + 1 : LayerId,
				KeyFrameGeometry,
				KeyDrawParams.FillBrush,
				KeyDrawEffects,
				KeyDrawParams.FillTint
			);

			// Draw border
			if (PaintArgs.KeyStyle == EKeyRendererStyle::FrameBlock)
			{
				const bool bFrameBlockAggregateOverlap = PaintArgs.KeyStyle == EKeyRendererStyle::FrameBlock
					&& Key.KeyTickStart != Key.KeyTickEnd;
				const bool bShowFrameBlockBorder = bSelected
					|| bPartiallySelected || bHovered || bOverlapped || bFrameBlockAggregateOverlap;
				if (bShowFrameBlockBorder)
				{
					FSlateDrawElement::MakeBox(
						*PaintArgs.DrawElements,
						bSelected ? LayerId + 2 : LayerId + 1,
						KeyFrameGeometry,
						KeyDrawParams.BorderBrush,
						KeyDrawEffects,
						KeyDrawParams.BorderTint
					);
				}
			}
		}
		else if (PaintArgs.KeyStyle == EKeyRendererStyle::Circle || PaintArgs.KeyStyle == EKeyRendererStyle::Diamond)
		{
			static constexpr float BrushBorderWidth = 2.f;

			if (PaintArgs.KeyStyle == EKeyRendererStyle::Diamond)
			{
				KeyDrawParams.FillBrush = FAppStyle::Get().GetBrush(TEXT("Sequencer.KeyDiamond"));
			}
			KeyDrawParams.ConnectionStyle = EKeyConnectionStyle::None;

			// draw border
			const FVector2D BorderOffset = FVector2D(
				KeyPositionPx - FMath::CeilToFloat(HalfKeySize.X),
				HalfLocalSize.Y - HalfKeySize.Y
			);

			FSlateDrawElement::MakeBox(
				*PaintArgs.DrawElements,
				// always draw selected keys on top of other keys
				bSelected ? LayerId + 1 : LayerId,
				// Center the key along Y.  Ensure the middle of the key is at the actual key time
				AllottedGeometry.ToPaintGeometry(
					KeySize,
					FSlateLayoutTransform(BorderOffset + PaintArgs.KeyOffset)
				),
				KeyDrawParams.FillBrush,
				KeyDrawEffects,
				KeyDrawParams.BorderTint
			);

			// draw fill
			const FVector2D FillOffset = FVector2D(
				KeyPositionPx - FMath::CeilToFloat(HalfKeySize.X - BrushBorderWidth),
				HalfLocalSize.Y - (HalfKeySize.Y - BrushBorderWidth)
			);

			FSlateDrawElement::MakeBox(
				*PaintArgs.DrawElements,
				// always draw selected keys on top of other keys
				bSelected ? LayerId + 2 : LayerId + 1,
				// Center the key along Y.  Ensure the middle of the key is at the actual key time
				AllottedGeometry.ToPaintGeometry(
					KeySize - (2.f * BrushBorderWidth),
					FSlateLayoutTransform(FillOffset + KeyDrawParams.FillOffset + PaintArgs.KeyOffset)
				),
				KeyDrawParams.FillBrush,
				KeyDrawEffects,
				KeyDrawParams.FillTint
			);
		}
	}

	return LayerId + 2;
}

EViewDependentCacheFlags FKeyRenderer::FCachedKeyDrawInformation::UpdateViewIndependentData()
{
	TViewModelPtr<IKeyExtension> KeyExtension = WeakKeyExtension.Pin();
	if (KeyExtension)
	{
		const bool bDataChanged = KeyExtension->UpdateCachedKeys(CachedKeys);

		return bDataChanged ? EViewDependentCacheFlags::DataChanged : EViewDependentCacheFlags::None;
	}
	else if (CachedKeys.IsValid())
	{
		CachedKeys = nullptr;
		return EViewDependentCacheFlags::DataChanged;
	}

	return EViewDependentCacheFlags::None;
}

void FKeyRenderer::FCachedKeyDrawInformation::CacheViewDependentData(const FKeyBatchParameters& Params)
{
	TViewModelPtr<IKeyExtension> KeyExtension = WeakKeyExtension.Pin();
	if (!KeyExtension)
	{
		CachedKeys = nullptr;

		DrawParams.Empty();
		FramesInRange = {};
		HandlesInRange = {};

		LeadingKey.Reset();
		TrailingKey.Reset();

		PreserveToIndex = FramesInRange.Num();
		NextUnhandledIndex = 0;
		return;
	}

	if (EnumHasAnyFlags(Params.CacheState, EViewDependentCacheFlags::DataChanged | EViewDependentCacheFlags::ViewChanged | EViewDependentCacheFlags::ViewZoomed))
	{
		TArrayView<const FFrameTime> OldFramesInRange = FramesInRange;

		FFrameTime KeyWidth = Params.TimeToPixel.PixelDeltaToFrame(Params.KeySizePx.X);

		// Dialte the visible range by the size of a key
		TRange<FFrameTime> DilatedVisibleRange = Params.VisibleRange;
		DilatedVisibleRange.SetLowerBoundValue(DilatedVisibleRange.GetLowerBoundValue() - KeyWidth*.5f);
		DilatedVisibleRange.SetUpperBoundValue(DilatedVisibleRange.GetUpperBoundValue() + KeyWidth*.5f);

		// Gather all the key handles in this view range, plus the next out-of-range key for connecting bars
		CachedKeys->GetKeysInRangeWithBounds(DilatedVisibleRange, &FramesInRange, &HandlesInRange, &LeadingKey, &TrailingKey);

		if (LeadingKey.IsValid())
		{
			KeyExtension->DrawKeys(MakeArrayView(&LeadingKey.Handle, 1), TArrayView<FKeyDrawParams>(&LeadingKeyParams, 1));
		}
		if (TrailingKey.IsValid())
		{
			KeyExtension->DrawKeys(MakeArrayView(&TrailingKey.Handle, 1), TArrayView<FKeyDrawParams>(&TrailingKeyParams, 1));
		}

		bool bDrawnKeys = false;
		if (!EnumHasAnyFlags(Params.CacheState, EViewDependentCacheFlags::DataChanged) && OldFramesInRange.Num() != 0 && FramesInRange.Num() != 0)
		{
			// Try and preserve draw params if possible
			const int32 PreserveStart = Algo::LowerBound(OldFramesInRange, FramesInRange[0]);
			const int32 PreserveEnd   = Algo::UpperBound(OldFramesInRange, FramesInRange.Last());

			const int32 PreserveNum   = PreserveEnd - PreserveStart;

			if (PreserveNum > 0)
			{
				const int32 HeadNum   = Algo::LowerBound(FramesInRange, OldFramesInRange[PreserveStart]);
				const int32 TailStart = Algo::LowerBound(FramesInRange, OldFramesInRange[PreserveEnd-1]);
				const int32 TailNum   = FramesInRange.Num() - TailStart;

				if (HeadNum == 0 && TailNum == 0)
				{
					// If we're preserving everything, just set the flag to avoid reallocating the whole array
					bDrawnKeys = true;
				}
				else
				{
					const int32 HeadPreserveDiff = HeadNum - PreserveStart;

					// Insert elements at the front of the array if we need to (this probably means we scrolled to the right to reveal new leading keys)
					if (HeadPreserveDiff > 0)
					{
						DrawParams.InsertUninitialized(0, HeadPreserveDiff);
					}
					// Remove elements at the front of the array if we need to (this probably means we scrolled to the left and hid leading keys)
					else if (HeadPreserveDiff < 0)
					{
						DrawParams.RemoveAt(0, -HeadPreserveDiff, EAllowShrinking::No);
					}

					// Draw newly revealed keys at the start
					if (HeadNum > 0)
					{
						KeyExtension->DrawKeys(HandlesInRange.Slice(0, HeadNum), TArrayView<FKeyDrawParams>(DrawParams).Slice(0, HeadNum));
					}

					// Expand or contract the array to the final size
					DrawParams.SetNum(FramesInRange.Num(), EAllowShrinking::No);

					// Draw newly revealed keys at the end
					if (TailNum > 0)
					{
						KeyExtension->DrawKeys(HandlesInRange.Slice(TailStart, TailNum), TArrayView<FKeyDrawParams>(DrawParams).Slice(TailStart, TailNum));
					}

					bDrawnKeys = true;
				}
			}
		}

		if (!bDrawnKeys)
		{
			DrawParams.SetNum(FramesInRange.Num(), EAllowShrinking::No);

			if (FramesInRange.Num())
			{
				// Draw these keys
				KeyExtension->DrawKeys(HandlesInRange, DrawParams);
			}
		}

		check(DrawParams.Num() == FramesInRange.Num() && FramesInRange.Num() == HandlesInRange.Num());

		// Shrink the draw params if we have excess slack
		if (DrawParams.GetSlack() > DrawParams.Num() / 3)
		{
			DrawParams.Shrink();
		}
	}

	// Always reset the pointers to the current key that needs processing
	PreserveToIndex = FramesInRange.Num();
	NextUnhandledIndex = 0;
}

} // namespace UE::Sequencer

