// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/OutlinerColumns/SColumnToggleWidget.h"

#include "Delegates/Delegate.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/SharedViewModelData.h"

#define LOCTEXT_NAMESPACE "SColumnToggleWidget"

namespace UE::Sequencer
{

class FColumnToggleDragDropOp : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FColumnToggleDragDropOp, FDragDropOperation)

	/** Flag which defines whether to hide destination items or not */
	bool bActive;

	/** Name of the column that is being dragged to not affect other columns */
	FName ColumnName;

	/** Widget determines event when drag drop operation is cancelled */
	TWeakPtr<SColumnToggleWidget> ColumnToggleWidget;

	/** Which is kept alive for the duration of the drag */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	/** Called when the drag drop operation ends */
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
	{
		if (TSharedPtr<SColumnToggleWidget> CurrentColumnToggleWidget = ColumnToggleWidget.Pin())
		{
			CurrentColumnToggleWidget->OnToggleOperationComplete();
		}
	}

	/** Create a new drag and drop operation out of the specified flag */
	static TSharedRef<FColumnToggleDragDropOp> New(const bool _bActive, FName _ColumnName, TWeakPtr<SColumnToggleWidget> _ColumnToggleWidget, TUniquePtr<FScopedTransaction>& ScopedTransaction)
	{
		TSharedRef<FColumnToggleDragDropOp> Operation = MakeShareable(new FColumnToggleDragDropOp);

		Operation->bActive = _bActive;
		Operation->ColumnName = _ColumnName;
		Operation->ColumnToggleWidget = _ColumnToggleWidget;
		Operation->UndoTransaction = MoveTemp(ScopedTransaction);

		Operation->Construct();
		return Operation;
	}
};

void SColumnToggleWidget::Construct(
	const FArguments& InArgs,
	const TWeakPtr<IOutlinerColumn> InOutlinerColumn,
	const FCreateOutlinerColumnParams& InParams)
{
	WeakOutlinerColumn = InOutlinerColumn;

	WeakOutlinerExtension = InParams.OutlinerExtension;
	WeakEditor = InParams.Editor;

	ModelID = InParams.OutlinerExtension.AsModel()->GetModelID();

	bIsMouseOverWidget = false;
	bIsActive = false;
	bIsChildActive = false;
	bIsImplicitlyActive = false;

	ActiveBrush = GetActiveBrush();
	InactiveBrush = GetInactiveBrush();

	SetCanTick(true);

	static const FName NAME_ChildActiveBrush = TEXT("Sequencer.Column.CheckBoxIndeterminate");
	ChildActiveBrush = FAppStyle::Get().GetBrush(NAME_ChildActiveBrush);

	SImage::Construct(
		SImage::FArguments()
		.Image(this, &SColumnToggleWidget::GetBrush)
		.IsEnabled(this, &SColumnToggleWidget::IsEnabled)
		.ColorAndOpacity(this, &SColumnToggleWidget::GetImageColorAndOpacity)
	);
}

void SColumnToggleWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	bIsActive = IsActive();
	bIsChildActive = IsChildActive();
	bIsImplicitlyActive = IsImplicitlyActive();
}

FSlateColor SColumnToggleWidget::GetImageColorAndOpacity() const
{
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();

	FLinearColor OutColor = FLinearColor::White;

	if (!Editor || !OutlinerItem)
	{
		return OutColor;
	}

	float Opacity = 0.0f;

	if (bIsActive)
	{
		// Directly active, full opacity
		Opacity = 1.0f;
	}
	else if (bIsChildActive && !bIsMouseOverWidget)
	{
		// Child is active and mouse is not over widget. Full opacity '-'.
		Opacity = 1.0f;
	}
	else if (bIsMouseOverWidget)
	{
		// Mouse is over widget and it is not directly active.
		Opacity = .8f;
	}
	else if (bIsImplicitlyActive)
	{
		// Implicitly active through another object and mouse is not over.
		Opacity = .35f;
	}
	else if (Editor->GetOutliner()->GetHoveredItem() == OutlinerItem)
	{
		// Mouse is hovered over outliner item and not the widget itself, preview icons for widget.
		Opacity = .25f;
	}
	else
	{
		// Not active in any way and mouse is not over widget or item.
		Opacity = 0.1f;
	}

	OutColor.A = Opacity;
	return OutColor;
}

const FSlateBrush* SColumnToggleWidget::GetBrush() const
{
	if (!bIsActive && !bIsImplicitlyActive)
	{
		if (bIsChildActive)
		{
			if (bIsMouseOverWidget)
			{
				return ActiveBrush;
			}

			return ChildActiveBrush;
		}

		if (InactiveBrush)
		{
			return InactiveBrush;
		}
	}

	return ActiveBrush;
}

FReply SColumnToggleWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<IOutlinerColumn> OutlinerColumn = WeakOutlinerColumn.Pin();
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton)
		&& OutlinerColumn)
	{
		return FReply::Handled().BeginDragDrop(FColumnToggleDragDropOp::New(IsActive(), OutlinerColumn->GetColumnName(), SharedThis(this), UndoTransaction));
	}
	else
	{
		return FReply::Unhandled();
	}
}

void SColumnToggleWidget::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	auto ColumnToggleDragOp = DragDropEvent.GetOperationAs<FColumnToggleDragDropOp>();
	TSharedPtr<IOutlinerColumn> OutlinerColumn = WeakOutlinerColumn.Pin();
	if (ColumnToggleDragOp.IsValid()
		&& OutlinerColumn
		&& ColumnToggleDragOp->ColumnName == OutlinerColumn->GetColumnName())
	{
		SetIsActive(ColumnToggleDragOp->bActive);
	}
}

FReply SColumnToggleWidget::HandleClick()
{
	TSharedPtr<IOutlinerColumn> OutlinerColumn = WeakOutlinerColumn.Pin();

	// Open an undo transaction
	if (OutlinerColumn)
	{
		UndoTransaction.Reset(new FScopedTransaction(FText::Format(LOCTEXT("Toggle Outliner Column", "Toggle {0}"), OutlinerColumn->GetColumnLabel())));
	}

	if (bIsActive)
	{
		SetIsActive(false);
	}
	else
	{
		SetIsActive(true);
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SColumnToggleWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

FReply SColumnToggleWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}


void SColumnToggleWidget::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	OnToggleOperationComplete();
	UndoTransaction.Reset();
}

FReply SColumnToggleWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		OnToggleOperationComplete();
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SColumnToggleWidget::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsMouseOverWidget = true;
}

void SColumnToggleWidget::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	bIsMouseOverWidget = false;
}

void SColumnToggleWidget::RefreshSequencerTree()
{
	if (TSharedPtr<FEditorViewModel> EditorViewModel = WeakEditor.Pin())
	{
		if (TSharedPtr<FOutlinerViewModel> Outliner = EditorViewModel->GetOutliner())
		{
			Outliner->RequestUpdate();
		}
	}
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
