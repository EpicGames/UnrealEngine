// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/OutlinerColumns/SDeactivateColumnWidget.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

#define LOCTEXT_NAMESPACE "SDeactivateColumnWidget"

namespace UE::Sequencer
{

void SDeactivateColumnWidget::OnToggleOperationComplete()
{
	// refresh the sequencer tree after operation is complete
	RefreshSequencerTree();
}

void SDeactivateColumnWidget::Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams)
{
	SetToolTipText(LOCTEXT("DeactivateTooltip", "Deactivate this track and disable evaluation.\n\n"
		"Saved with the asset."));

	SColumnToggleWidget::Construct(
		SColumnToggleWidget::FArguments(),
		InWeakOutlinerColumn,
		InParams
	);

	WeakStateCacheExtension = CastViewModel<FDeactiveStateCacheExtension>(InParams.OutlinerExtension.AsModel()->GetSharedData());
}

bool SDeactivateColumnWidget::IsActive() const
{
	if (const TViewModelPtr<FDeactiveStateCacheExtension> StateCache = WeakStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedDeactiveState::Deactivated);
	}

	return false;
}

void SDeactivateColumnWidget::SetIsActive(const bool bInIsActive)
{
	const TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();
	const TSharedPtr<FEditorViewModel>      Editor       = WeakEditor.Pin();
	if (!OutlinerItem || !Editor)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetNodeDeactivated", "Set Node Deactivated"));

	if (OutlinerItem->GetSelectionState() == EOutlinerSelectionState::SelectedDirectly)
	{
		// If selected, modify all selected items
		for (const FViewModelPtr Selected : *Editor->GetSelection()->GetOutlinerSelection())
		{
			SetIsActive_Internal(Selected, bInIsActive);
		}
	}
	else
	{
		SetIsActive_Internal(OutlinerItem.AsModel(), bInIsActive);
	}
}

void SDeactivateColumnWidget::SetIsActive_Internal(const FViewModelPtr& ViewModel, const bool bInIsActive)
{
	// If this is deactivatable, deactivate only this
	if (const TViewModelPtr<IDeactivatableExtension> Deactivatable = ViewModel.ImplicitCast())
	{
		Deactivatable->SetIsDeactivated(bInIsActive);
	}
	// Otherwise deactivate deactivatable children of this (if any)
	else for (const TViewModelPtr<IDeactivatableExtension>& Child : ViewModel->GetDescendantsOfType<IDeactivatableExtension>())
	{
		if (Child->IsInheritable())
		{
			Child->SetIsDeactivated(bInIsActive);
		}
	}
}

bool SDeactivateColumnWidget::IsChildActive() const
{
	if (const TViewModelPtr<FDeactiveStateCacheExtension> StateCache = WeakStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedDeactiveState::PartiallyDeactivatedChildren);
	}

	return false;
}

bool SDeactivateColumnWidget::IsImplicitlyActive() const
{
	if (const TViewModelPtr<FDeactiveStateCacheExtension> StateCache = WeakStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedDeactiveState::ImplicitlyDeactivatedByParent);
	}

	return false;
}

const FSlateBrush* SDeactivateColumnWidget::GetActiveBrush() const
{
	if (const TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin())
	{
		bool bHasTrackAreaModels = false;

		if (const TViewModelPtr<ITrackAreaExtension> TrackArea = OutlinerItem.ImplicitCast())
		{
			if (const auto DeactivatableTrackArea = TrackArea->GetTrackAreaModelList())
			{
				bHasTrackAreaModels = true;
			}
		}

		if (!bHasTrackAreaModels)
		{
			if (const TViewModelPtr<IDeactivatableExtension> Deactivatable = OutlinerItem.ImplicitCast())
			{
				const TParentFirstChildIterator<IDeactivatableExtension> DeactivatableDescendants
					= OutlinerItem.AsModel()->GetDescendantsOfType<IDeactivatableExtension>();
				if (!DeactivatableDescendants)
				{
					static const FName PartialBrushName = TEXT("Sequencer.Column.CheckBoxIndeterminate");
					return FAppStyle::Get().GetBrush(PartialBrushName);
				}
			}
		}
	}

	static const FName BrushName = TEXT("Sequencer.Column.Mute");
	return FAppStyle::Get().GetBrush(BrushName);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
