// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/OutlinerColumns/SColumnToggleWidget.h"
#include "SequencerCoreFwd.h"

namespace UE::Sequencer
{

class FDeactiveStateCacheExtension;

/**
 * A widget that shows and controls the deactivated state of outliner items.
 */
class SDeactivateColumnWidget
	: public SColumnToggleWidget
{
public:
	SLATE_BEGIN_ARGS(SDeactivateColumnWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams);

	/** Refreshes the Sequencer Tree. */
	virtual void OnToggleOperationComplete();

protected:
	//~ Begin SColumnToggleWidget

	/** Returns whether or not this item is deactivated. */
	virtual bool IsActive() const override;

	/** Sets this item as activated or deactivated. If selected, applies to all selected items. */
	virtual void SetIsActive(const bool bInIsActive) override;

	/** Returns whether or not a child of this item is deactivated. */
	virtual bool IsChildActive() const override;

	/** Returns true if this item is implicitly deactivated, but not directly deactivated */
	virtual bool IsImplicitlyActive() const override;

	/** Returns the brush to display when this item is deactivated. */
	virtual const FSlateBrush* GetActiveBrush() const override;

	//~ End SColumnToggleWidget

private:
	void SetIsActive_Internal(const FViewModelPtr& ViewModel, const bool bInIsActive);

	/** Weak cache extension ptr (can be null). */
	TWeakViewModelPtr<FDeactiveStateCacheExtension> WeakStateCacheExtension;
};

} // namespace UE::Sequencer
