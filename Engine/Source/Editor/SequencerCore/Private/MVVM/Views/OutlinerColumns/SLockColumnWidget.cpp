// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/OutlinerColumns/SLockColumnWidget.h"

#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"

namespace UE::Sequencer
{

#define LOCTEXT_NAMESPACE "SLockColumnWidget"

void SLockColumnWidget::Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams)
{
	SetToolTipText(LOCTEXT("LockTooltip", "Lock this track and disable editing.\n\n"
		"Saved with the asset."));

	SColumnToggleWidget::Construct(
		SColumnToggleWidget::FArguments(),
		InWeakOutlinerColumn,
		InParams
	);

	WeakLockStateCacheExtension = CastViewModel<FLockStateCacheExtension>(InParams.OutlinerExtension.AsModel()->GetSharedData());
}

bool SLockColumnWidget::IsActive() const
{
	if (TViewModelPtr<FLockStateCacheExtension> StateCache = WeakLockStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedLockState::Locked);
	}

	return false;
}

void SLockColumnWidget::SetIsActive(const bool bInIsActive)
{
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();
	TSharedPtr<FEditorViewModel>      Editor       = WeakEditor.Pin();
	if (!OutlinerItem || !Editor)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetNodeLocked", "Set Node Locked"));

	if (OutlinerItem->GetSelectionState() == EOutlinerSelectionState::SelectedDirectly)
	{
		// modify all selected items
		for (FViewModelPtr OutlinerNode : *Editor->GetSelection()->GetOutlinerSelection())
		{
			for (TSharedPtr<ILockableExtension> Lockable : OutlinerNode->GetDescendantsOfType<ILockableExtension>(true))
			{
				Lockable->SetIsLocked(bInIsActive);
			}
		}
	}
	else
	{
		// only one unselected item was toggled, toggle just that node
		FViewModelPtr Item = OutlinerItem;
		for (TSharedPtr<ILockableExtension> Lockable : Item->GetDescendantsOfType<ILockableExtension>(true))
		{
			Lockable->SetIsLocked(bInIsActive);
		}
	}
}

bool SLockColumnWidget::IsChildActive() const
{
	if (TViewModelPtr<FLockStateCacheExtension> StateCache = WeakLockStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedLockState::PartiallyLockedChildren);
	}

	return false;
}

const FSlateBrush* SLockColumnWidget::GetActiveBrush() const
{
	static const FName NAME_LockBrush = TEXT("Sequencer.Column.Locked");
	return FAppStyle::Get().GetBrush(NAME_LockBrush);
}

#undef LOCTEXT_NAMESPACE

} // namespace UE::Sequencer
