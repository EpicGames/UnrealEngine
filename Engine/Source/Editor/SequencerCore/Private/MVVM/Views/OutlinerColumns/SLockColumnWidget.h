// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/OutlinerColumns/SColumnToggleWidget.h"

namespace UE::Sequencer
{

class FLockStateCacheExtension;

/**
* A widget that shows and controls the locked state of outliner items.
*/
class SLockColumnWidget
	: public SColumnToggleWidget
{
public:
	SLATE_BEGIN_ARGS(SLockColumnWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams);

public:
	/** Returns whether or not this outliner item is locked. */
	virtual bool IsActive() const override;

	/** Sets this item and it's sections as locked or unlocked. If selected, applies to all selected items. */
	virtual void SetIsActive(const bool bInIsActive) override;

	/** Returns whether or not a child of this item is locked. */
	virtual bool IsChildActive() const override;

	/** Returns the brush to display when this item is locked. */
	virtual const FSlateBrush* GetActiveBrush() const override;

private:

	/** Weak cache extension ptr (can be null). */
	TWeakViewModelPtr<FLockStateCacheExtension> WeakLockStateCacheExtension;
};

} // namespace UE::Sequencer