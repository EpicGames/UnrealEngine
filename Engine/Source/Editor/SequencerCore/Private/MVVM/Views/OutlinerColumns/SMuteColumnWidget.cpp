// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/OutlinerColumns/SMuteColumnWidget.h"

#include "MVVM/SharedViewModelData.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"

namespace UE::Sequencer
{

#define LOCTEXT_NAMESPACE "SMuteColumnWidget"

void SMuteColumnWidget::OnToggleOperationComplete()
{
	// refresh the sequencer tree after operation is complete
	RefreshSequencerTree();
}

void SMuteColumnWidget::Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams)
{
	SetToolTipText(LOCTEXT("MuteTooltip", "Mute this track and disable evaluation locally.\n\n"
		"*Not* saved with the asset, but saved in memory until editor restart"));

	SColumnToggleWidget::Construct(
		SColumnToggleWidget::FArguments(),
		InWeakOutlinerColumn,
		InParams
	);

	WeakMuteStateCacheExtension = CastViewModel<FMuteStateCacheExtension>(InParams.OutlinerExtension.AsModel()->GetSharedData());
}

bool SMuteColumnWidget::IsActive() const
{
	if (TViewModelPtr<FMuteStateCacheExtension> MuteStateCache = WeakMuteStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(MuteStateCache->GetCachedFlags(ModelID), ECachedMuteState::Muted);
	}

	return false;
}

void SMuteColumnWidget::SetIsActive(const bool bInIsActive)
{
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();
	TSharedPtr<FEditorViewModel>      Editor       = WeakEditor.Pin();
	if (!OutlinerItem || !Editor)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetNodeMuted", "Set Node Muted"));

	if (OutlinerItem->GetSelectionState() == EOutlinerSelectionState::SelectedDirectly)
	{
		// If selected, modify all selected items
		for (FViewModelPtr Selected : *Editor->GetSelection()->GetOutlinerSelection())
		{
			SetIsActive_Internal(Selected, bInIsActive);
		}
	}
	else
	{
		SetIsActive_Internal(OutlinerItem.AsModel(), bInIsActive);
	}
}

void SMuteColumnWidget::SetIsActive_Internal(const FViewModelPtr& ViewModel, const bool bInIsActive)
{
	// If this is mutable, mute only this
	if (const TViewModelPtr<IMutableExtension> Mutable = ViewModel.ImplicitCast())
	{
		Mutable->SetIsMuted(bInIsActive);
	}
	// Otherwise mute mutable children of this (if any)
	else for (TViewModelPtr<IMutableExtension> Child : ViewModel->GetDescendantsOfType<IMutableExtension>())
	{
		if (Child->IsInheritable())
		{
			Child->SetIsMuted(bInIsActive);
		}
	}
}

bool SMuteColumnWidget::IsChildActive() const
{
	if (TViewModelPtr<FMuteStateCacheExtension> MuteStateCache = WeakMuteStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(MuteStateCache->GetCachedFlags(ModelID), ECachedMuteState::PartiallyMutedChildren);
	}

	return false;
}

bool SMuteColumnWidget::IsImplicitlyActive() const
{
	if (TViewModelPtr<FMuteStateCacheExtension> MuteStateCache = WeakMuteStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(MuteStateCache->GetCachedFlags(ModelID), ECachedMuteState::ImplicitlyMutedByParent);
	}

	return false;
}

const FSlateBrush* SMuteColumnWidget::GetActiveBrush() const
{
	static const FName NAME_MutedBrush = TEXT("Level.NotVisibleIcon16x");
	return FAppStyle::Get().GetBrush(NAME_MutedBrush);
}

const FSlateBrush* SMuteColumnWidget::GetInactiveBrush() const
{
	static const FName NAME_NotMutedBrush = TEXT("Level.VisibleIcon16x");
	return FAppStyle::Get().GetBrush(NAME_NotMutedBrush);
}

#undef LOCTEXT_NAMESPACE

} // namespace UE::Sequencer
