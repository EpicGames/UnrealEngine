// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/OutlinerColumns/SColumnToggleWidget.h"

namespace UE::Sequencer
{

class FMuteStateCacheExtension;

/**
 * A widget that shows and controls the muted state of outliner items.
 */
class SMuteColumnWidget
	: public SColumnToggleWidget
{
public:
	SLATE_BEGIN_ARGS(SMuteColumnWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams);

public:

	/** Refreshes the Sequencer Tree to handle simultaneous mute and solo modifiers. */
	virtual void OnToggleOperationComplete();

protected:
	/** Returns whether or not this item is muted. */
	virtual bool IsActive() const override;

	/** Sets this item as muted or unmuted. If selected, applies to all selected items. */
	virtual void SetIsActive(const bool bInIsActive) override;

	/** Returns whether or not a child of this item is muted. */
	virtual bool IsChildActive() const override;

	/** Returns true if this item is implicitly active, but not directly active */
	virtual bool IsImplicitlyActive() const override;

	/** Returns the brush to display when this item is muted. */
	virtual const FSlateBrush* GetActiveBrush() const override;

	/** Returns the brush to display when this item is not muted. */
	virtual const FSlateBrush* GetInactiveBrush() const override;

private:

	void SetIsActive_Internal(const FViewModelPtr& ViewModel, const bool bInIsActive);

	/** Weak cache extension ptr (can be null). */
	TWeakViewModelPtr<FMuteStateCacheExtension> WeakMuteStateCacheExtension;
};

} // namespace UE::Sequencer