// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/OutlinerColumns/SOutlinerColumnButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"

namespace UE::Sequencer
{

void SOutlinerColumnButton::Construct(const FArguments& InArgs)
{
	IsRowHovered = InArgs._IsRowHovered;
	IsChecked = InArgs._IsChecked;
	Image = InArgs._Image;
	UncheckedImage = InArgs._UncheckedImage;

	TSharedRef<SButton> Button = SNew(SButton)
		.IsFocusable(InArgs._IsFocusable)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ContentPadding(FMargin(0.f))
		.ButtonStyle(FAppStyle::Get(), "Sequencer.Outliner.ColumnButton")
		.ToolTipText(InArgs._ToolTipText)
		[
			SNew(SImage)
			.ColorAndOpacity(this, &SOutlinerColumnButton::GetColorAndOpacity)
			.Image(this, &SOutlinerColumnButton::GetImage)
		];

	if (InArgs._OnGetMenuContent.IsBound())
	{
		Button->SetOnClicked(FOnClicked::CreateSP(this, &SOutlinerColumnButton::ToggleMenu));

		ChildSlot
		[
			SAssignNew(MenuAnchor, SMenuAnchor)
			.OnGetMenuContent(InArgs._OnGetMenuContent)
			[
				Button
			]
		];
	}
	else
	{
		Button->SetOnClicked(InArgs._OnClicked);
		ChildSlot
		[
			Button
		];
	}
}

const FSlateBrush* SOutlinerColumnButton::GetImage() const
{
	if (IsChecked.IsBound())
	{
		return IsChecked.Get() ? Image.Get() : UncheckedImage.Get();
	}
	return Image.Get();
}

FSlateColor SOutlinerColumnButton::GetColorAndOpacity() const
{
	const uint8 IntIsRowHovered    = uint8(IsRowHovered.Get(false));   // 0 or 1
	const uint8 IntIsButtonHovered = uint8(IsHovered()) << 1;          // 0 or 2

	static constexpr FLinearColor Colors[] = {
		FLinearColor(1.f, 1.f, 1.f, 0.1f),	// NoHover
		FLinearColor(1.f, 1.f, 1.f, 0.25f),	// Row Hover
		FLinearColor(1.f, 1.f, 1.f, 0.25f),	// Button Hover
		FLinearColor(1.f, 1.f, 1.f, 0.8f)	// Both Hovered
	};

	return Colors[IntIsRowHovered | IntIsButtonHovered];
}

FReply SOutlinerColumnButton::ToggleMenu()
{
	MenuAnchor->SetIsOpen(!MenuAnchor->IsOpen());
	return FReply::Handled();
}


} // namespace UE::Sequencer