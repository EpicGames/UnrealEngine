// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/OutlinerColumns/SSoloColumnWidget.h"

#include "MVVM/SharedViewModelData.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/Extensions/ISoloableExtension.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"

namespace UE::Sequencer
{

#define LOCTEXT_NAMESPACE "SSoloColumnWidget"

void SSoloColumnWidget::OnToggleOperationComplete()
{
	// refresh the sequencer tree after operation is complete
	RefreshSequencerTree();
}

void SSoloColumnWidget::Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams)
{
	SetToolTipText(LOCTEXT("SoloTooltip", "Solo this track and disable evaluation for all other tracks locally.\n\n"
		"*Not* saved with the asset, but saved in memory until editor restart"));

	SColumnToggleWidget::Construct(
		SColumnToggleWidget::FArguments(),
		InWeakOutlinerColumn,
		InParams
	);

	WeakSoloStateCacheExtension = CastViewModel<FSoloStateCacheExtension>(InParams.OutlinerExtension.AsModel()->GetSharedData());
}

bool SSoloColumnWidget::IsActive() const
{
	if (TViewModelPtr<FSoloStateCacheExtension> SoloStateCache = WeakSoloStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(SoloStateCache->GetCachedFlags(ModelID), ECachedSoloState::Soloed);
	}

	return false;
}

void SSoloColumnWidget::SetIsActive(const bool bInIsActive)
{
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();
	TSharedPtr<FEditorViewModel>      Editor       = WeakEditor.Pin();
	if (!OutlinerItem || !Editor)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetNodeSoloed", "Set Node Soloed"));

	if (OutlinerItem->GetSelectionState() == EOutlinerSelectionState::SelectedDirectly)
	{
		// if selected, modify all selected items
		for (TViewModelPtr<ISoloableExtension> Soloable : Editor->GetSelection()->GetOutlinerSelection()->Filter<ISoloableExtension>())
		{
			Soloable->SetIsSoloed(bInIsActive);
		}
	}
	else if (TViewModelPtr<ISoloableExtension> Soloable = OutlinerItem.ImplicitCast())
	{
		Soloable->SetIsSoloed(bInIsActive);
	}
}

bool SSoloColumnWidget::IsChildActive() const
{
	if (TViewModelPtr<FSoloStateCacheExtension> SoloStateCache = WeakSoloStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(SoloStateCache->GetCachedFlags(ModelID), ECachedSoloState::PartiallySoloedChildren);
	}

	return false;
}

bool SSoloColumnWidget::IsImplicitlyActive() const
{
	if (TViewModelPtr<FSoloStateCacheExtension> SoloStateCache = WeakSoloStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(SoloStateCache->GetCachedFlags(ModelID), ECachedSoloState::ImplicitlySoloedByParent);
	}

	return false;
}

const FSlateBrush* SSoloColumnWidget::GetActiveBrush() const
{
	static const FName NAME_SoloedBrush = TEXT("Sequencer.Column.Solo");
	return FAppStyle::Get().GetBrush(NAME_SoloedBrush);
}

#undef LOCTEXT_NAMESPACE

} // namespace UE::Sequencer
