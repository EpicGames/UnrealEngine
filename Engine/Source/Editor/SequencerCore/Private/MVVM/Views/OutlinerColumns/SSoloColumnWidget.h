// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/OutlinerColumns/SColumnToggleWidget.h"

namespace UE::Sequencer
{

class FSoloStateCacheExtension;

/**
* A widget that shows and controls the soloed state of outliner items.
*/
class SSoloColumnWidget
	: public SColumnToggleWidget
{
public:
	SLATE_BEGIN_ARGS(SSoloColumnWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams);

public:

	/** Refreshes the Sequencer Tree to show other outliner items as greyed out if they are not also soloed. */
	virtual void OnToggleOperationComplete();

protected:

	/** Returns whether or not this outliner item is soloed. */
	virtual bool IsActive() const override;

	/** Sets this item as soloed or not soloed. If selected, applies to all selected items. */
	virtual void SetIsActive(const bool bInIsActive) override;

	/** Returns whether or not a child of this item is soloed. */
	virtual bool IsChildActive() const override;

	/** Returns true if this item is implicitly active, but not directly active */
	virtual bool IsImplicitlyActive() const override;

	/** Returns the brush to display when this item is pinned. */
	virtual const FSlateBrush* GetActiveBrush() const override;

private:

	/** Weak cache extension ptr (can be null). */
	TWeakViewModelPtr<FSoloStateCacheExtension> WeakSoloStateCacheExtension;
};

} // namespace UE::Sequencer