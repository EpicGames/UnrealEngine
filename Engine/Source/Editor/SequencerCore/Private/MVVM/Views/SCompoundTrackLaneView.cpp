// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SCompoundTrackLaneView.h"
#include "MVVM/Views/STrackLane.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "Widgets/SWeakWidget.h"

namespace UE::Sequencer
{

SCompoundTrackLaneView::FSlot::FSlot(TSharedPtr<ITrackLaneWidget> InInterface, TWeakPtr<STrackLane> InOwningLane)
	: Interface(InInterface)
	, WeakOwningLane(InOwningLane)
{}
SCompoundTrackLaneView::FSlot::FSlot(TWeakPtr<ITrackLaneWidget> InWeakInterface, TWeakPtr<STrackLane> InOwningLane)
	: WeakInterface(InWeakInterface)
	, WeakOwningLane(InOwningLane)
{}

SCompoundTrackLaneView::SCompoundTrackLaneView()
	: Children(this)
{
}

SCompoundTrackLaneView::~SCompoundTrackLaneView()
{
}

void SCompoundTrackLaneView::Construct(const FArguments& InArgs, TSharedPtr<ITrackLaneWidgetSpace> InTrackLaneWidgetSpace)
{
	SetClipping(EWidgetClipping::ClipToBounds);

	WeakTrackLaneWidgetSpace = InTrackLaneWidgetSpace;
}

void SCompoundTrackLaneView::AddWeakWidget(TSharedPtr<ITrackLaneWidget> InWidget, TWeakPtr<STrackLane> InOwningLane)
{
	FSlot::FSlotArguments SlotArguments(MakeUnique<FSlot>(TWeakPtr<ITrackLaneWidget>(InWidget), InOwningLane));
	SlotArguments.AttachWidget(
		SNew(SWeakWidget).PossiblyNullContent(InWidget->AsWidget())
	);
	Children.AddSlot(MoveTemp(SlotArguments));
}

void SCompoundTrackLaneView::AddStrongWidget(TSharedPtr<ITrackLaneWidget> InWidget, TWeakPtr<STrackLane> InOwningLane)
{
	FSlot::FSlotArguments SlotArguments(MakeUnique<FSlot>(InWidget, InOwningLane));
	SlotArguments.AttachWidget(InWidget->AsWidget());
	Children.AddSlot(MoveTemp(SlotArguments));
}

void SCompoundTrackLaneView::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	TSharedPtr<ITrackLaneWidgetSpace> WidgetSpace = WeakTrackLaneWidgetSpace.Pin();
	if (!WidgetSpace)
	{
		return;
	}

	for (int32 WidgetIndex = 0; WidgetIndex < Children.Num(); ++WidgetIndex)
	{
		const FSlot&        Slot   = Children[WidgetIndex];
		TSharedRef<SWidget> Widget = Slot.GetWidget();

		EVisibility WidgetVisibility = Widget->GetVisibility();
		if (ArrangedChildren.Accepts(WidgetVisibility))
		{
			TSharedPtr<ITrackLaneWidget> Interface = Slot.GetInterface();
			TSharedPtr<STrackLane>       TrackLane = Slot.WeakOwningLane.Pin();

			if (!Interface || !TrackLane)
			{
				continue;
			}

			FGeometry LaneGeometry = AllottedGeometry;

			if (TSharedPtr<STrackLane> ParentLane = TrackLane->GetParentLane())
			{
				// If we have a parent track lane, make the geometry relative to that
				LaneGeometry = AllottedGeometry.MakeChild(
					FVector2D(AllottedGeometry.GetLocalSize().X, TrackLane->GetDesiredSize().Y),
					FSlateLayoutTransform(FVector2D(0.f, TrackLane->GetVerticalPosition() - ParentLane->GetVerticalPosition()))
				);
			}

			FTrackLaneScreenAlignment ScreenAlignment = Interface->GetAlignment(*WidgetSpace, LaneGeometry);
			if (ScreenAlignment.IsVisible())
			{
				FArrangedWidget ArrangedWidget = ScreenAlignment.ArrangeWidget(Widget, LaneGeometry);
				ArrangedChildren.AddWidget(WidgetVisibility, ArrangedWidget);
			}
		}
	}
}

FVector2D SCompoundTrackLaneView::ComputeDesiredSize(float) const
{
	return FVector2D(10.f, 10.f);
}

FChildren* SCompoundTrackLaneView::GetChildren()
{
	return &Children;
}

int32 SCompoundTrackLaneView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	SPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	return LayerId;
}

} // namespace UE::Sequencer

