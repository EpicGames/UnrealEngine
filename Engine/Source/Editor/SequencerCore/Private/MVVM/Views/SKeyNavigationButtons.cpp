// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SKeyNavigationButtons.h"

#include "Containers/UnrealString.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Extensions/IHoveredExtension.h"
#include "Math/NumericLimits.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "MVVM/Views/OutlinerColumns/SOutlinerColumnButton.h"


struct FSlateBrush;

namespace UE
{
namespace Sequencer
{

void SKeyNavigationButtons::Construct(const FArguments& InArgs, const TSharedPtr<FViewModel>& InModel)
{
	WeakModel = InModel;

	TAttribute<bool> IsHovered;
	if (TSharedPtr<IHoveredExtension> Hoverable = InModel->CastThisShared<IHoveredExtension>())
	{
		IsHovered = MakeAttributeSP(Hoverable.ToSharedRef(), &IHoveredExtension::IsHovered);
	}

	GetNavigatableTimesEvent = InArgs._GetNavigatableTimes;
	AddKeyEvent = InArgs._OnAddKey;
	SetTimeEvent = InArgs._OnSetTime;
	TimeAttribute = InArgs._Time;

	TSharedRef<SHorizontalBox> BoxPanel = SNew(SHorizontalBox);

	const float CommonPadding = 4.f;

	if (EnumHasAnyFlags(InArgs._Buttons, EKeyNavigationButtons::PreviousKey))
	{
		BoxPanel->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(CommonPadding, 0.f)
		.AutoWidth()
		[
			SNew(SOutlinerColumnButton)
			.IsFocusable(false)
			.IsRowHovered(IsHovered)
			.ToolTipText(InArgs._PreviousKeyToolTip)
			.Image(FAppStyle::GetBrush("Sequencer.Outliner.PreviousKey"))
			.OnClicked(this, &SKeyNavigationButtons::OnPreviousKeyClicked)
		];
	}

	if (EnumHasAnyFlags(InArgs._Buttons, EKeyNavigationButtons::AddKey))
	{
		BoxPanel->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(CommonPadding, 0.f)
		.AutoWidth()
		[
			SNew(SOutlinerColumnButton)
			.IsFocusable(false)
			.IsRowHovered(IsHovered)
			.ToolTipText(InArgs._AddKeyToolTip)
			.Image(FAppStyle::GetBrush("Sequencer.Outliner.AddKey"))
			.OnClicked(this, &SKeyNavigationButtons::OnAddKeyClicked)
		];
	}

	if (EnumHasAnyFlags(InArgs._Buttons, EKeyNavigationButtons::NextKey))
	{
		BoxPanel->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(CommonPadding, 0.f)
		.AutoWidth()
		[
			SNew(SOutlinerColumnButton)
			.IsFocusable(false)
			.IsRowHovered(IsHovered)
			.ToolTipText(InArgs._NextKeyToolTip)
			.Image(FAppStyle::GetBrush("Sequencer.Outliner.NextKey"))
			.OnClicked(this, &SKeyNavigationButtons::OnNextKeyClicked)
		];
	}

	if (BoxPanel->NumSlots() > 0)
	{
		FMargin LeadingPadding = BoxPanel->GetSlot(0).GetPadding();
		LeadingPadding.Left += CommonPadding;
		BoxPanel->GetSlot(0).SetPadding(LeadingPadding);

		FMargin TrailingPadding = BoxPanel->GetSlot(BoxPanel->NumSlots()-1).GetPadding();
		TrailingPadding.Right += CommonPadding;
		BoxPanel->GetSlot(BoxPanel->NumSlots()-1).SetPadding(TrailingPadding);
	}
	ChildSlot
	[
		BoxPanel
	];
}

FReply SKeyNavigationButtons::OnPreviousKeyClicked()
{
	FFrameTime ClosestPreviousKeyDistance = FFrameTime(TNumericLimits<int32>::Max(), 0.99999f);
	FFrameTime CurrentTime = TimeAttribute.Get();
	TOptional<FFrameTime> PreviousTime;

	TArray<FFrameNumber> AllTimes;
	GetNavigatableTimesEvent.ExecuteIfBound(AllTimes);
	if (AllTimes.IsEmpty())
	{
		return FReply::Handled();
	}

	for (FFrameNumber Time : AllTimes)
	{
		if (Time < CurrentTime && CurrentTime - Time < ClosestPreviousKeyDistance)
		{
			PreviousTime = Time;
			ClosestPreviousKeyDistance = CurrentTime - Time;
		}
	}

	if (!PreviousTime.IsSet() && AllTimes.Num() > 0)
	{
		AllTimes.Sort();

		PreviousTime = AllTimes.Last();
	}

	if (PreviousTime.IsSet())
	{
		SetTimeEvent.ExecuteIfBound(PreviousTime.GetValue());
	}
	return FReply::Handled();
}

FReply SKeyNavigationButtons::OnNextKeyClicked()
{
	FFrameTime ClosestNextKeyDistance = FFrameTime(TNumericLimits<int32>::Max(), 0.99999f);
	FFrameTime CurrentTime = TimeAttribute.Get();
	TOptional<FFrameTime> NextTime;

	TArray<FFrameNumber> AllTimes;
	GetNavigatableTimesEvent.ExecuteIfBound(AllTimes);
	if (AllTimes.IsEmpty())
	{
		return FReply::Handled();
	}

	for (FFrameNumber Time : AllTimes)
	{
		if (Time > CurrentTime && Time - CurrentTime < ClosestNextKeyDistance)
		{
			NextTime = Time;
			ClosestNextKeyDistance = Time - CurrentTime;
		}
	}

	if (!NextTime.IsSet() && AllTimes.Num() > 0)
	{
		AllTimes.Sort();

		NextTime = AllTimes[0];
	}

	if (NextTime.IsSet())
	{
		SetTimeEvent.ExecuteIfBound(NextTime.GetValue());
	}

	return FReply::Handled();
}

FReply SKeyNavigationButtons::OnAddKeyClicked()
{
	if (TSharedPtr<FViewModel> DataModel = WeakModel.Pin())
	{
		FFrameTime CurrentTime = TimeAttribute.Get();
		AddKeyEvent.ExecuteIfBound(CurrentTime, DataModel);
	}

	return FReply::Handled();
}

} // namespace Sequencer
} // namespace UE

