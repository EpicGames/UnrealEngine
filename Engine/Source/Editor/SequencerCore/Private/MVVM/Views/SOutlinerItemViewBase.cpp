// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerItemViewBase.h"

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "MVVM/Extensions/IDimmableExtension.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "SequencerCoreFwd.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Types/SlateStructs.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

struct FGeometry;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "SOutlinerItemViewBase"

namespace UE::Sequencer
{

void SOutlinerItemViewBase::Construct(
		const FArguments& InArgs,
		TWeakViewModelPtr<IOutlinerExtension> InWeakExtension,
		TWeakPtr<FEditorViewModel> InWeakEditor,
		const TSharedRef<ISequencerTreeViewRow>& InTableRow )
{
	WeakOutlinerExtension = InWeakExtension;
	WeakRenameExtension   = InWeakExtension.ImplicitPin();
	WeakEditor            = InWeakEditor;

	ItemStyle              = InArgs._ItemStyle;
	IsReadOnlyAttribute    = InArgs._IsReadOnly;
	IsRowSelectedAttribute = MakeAttributeSP(&InTableRow.Get(), &ISequencerTreeViewRow::IsItemSelected);

	if (!IsReadOnlyAttribute.IsSet())
	{
		IsReadOnlyAttribute = MakeAttributeSP(InWeakEditor.Pin().ToSharedRef(), &FEditorViewModel::IsReadOnly);
	}

	TViewModelPtr<IOutlinerExtension> OutlinerExtension = WeakOutlinerExtension.Pin();
	checkf(OutlinerExtension, TEXT("Attempting to create an outliner node from a null model"));

	if (ItemStyle == EOutlinerItemViewBaseStyle::ContainerHeader)
	{
		ExpandedBackgroundBrush = FAppStyle::GetBrush( "Sequencer.AnimationOutliner.TopLevelBorder_Expanded" );
		CollapsedBackgroundBrush = FAppStyle::GetBrush( "Sequencer.AnimationOutliner.TopLevelBorder_Collapsed" );
	}
	else
	{
		ExpandedBackgroundBrush = FAppStyle::GetBrush( "Sequencer.AnimationOutliner.DefaultBorder" );
		CollapsedBackgroundBrush = FAppStyle::GetBrush( "Sequencer.AnimationOutliner.DefaultBorder" );
	}

	FMargin InnerNodePadding;
	if (ItemStyle == EOutlinerItemViewBaseStyle::InsideContainer)
	{
		InnerBackgroundBrush = FAppStyle::GetBrush( "Sequencer.AnimationOutliner.TopLevelBorder_Expanded" );
		InnerNodePadding = FMargin(0.f, 1.f);
	}
	else
	{
		InnerBackgroundBrush = FAppStyle::GetBrush( "Sequencer.AnimationOutliner.TransparentBorder" );
		InnerNodePadding = FMargin(0.f);
	}

	TableRowStyle = &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("Sequencer.Outliner.Row");

	TSharedPtr<SWidget> LabelContent;

	if (TViewModelPtr<IRenameableExtension> Renameable = WeakRenameExtension.Pin())
	{
		TSharedRef<SInlineEditableTextBlock> EditableLabel = SNew(SInlineEditableTextBlock)
		.IsReadOnly(this, &SOutlinerItemViewBase::IsNodeLabelReadOnly)
		.Font(this, &SOutlinerItemViewBase::GetLabelFont)
		.Text(this, &SOutlinerItemViewBase::GetLabel)
		.ToolTipText(this, &SOutlinerItemViewBase::GetLabelToolTipText)
		.ColorAndOpacity(this, &SOutlinerItemViewBase::GetLabelColor)
		.OnVerifyTextChanged(this, &SOutlinerItemViewBase::IsRenameValid)
		.OnTextCommitted(this, &SOutlinerItemViewBase::OnNodeLabelTextCommitted)
		.Clipping(EWidgetClipping::ClipToBounds)
		.IsSelected(FIsSelected::CreateSP(&InTableRow.Get(), &ISequencerTreeViewRow::IsSelectedExclusively));

		Renameable->OnRenameRequested().AddSP(EditableLabel, &SInlineEditableTextBlock::EnterEditingMode);
		LabelContent = EditableLabel;
	}
	else
	{
		LabelContent = SNew(STextBlock)
		.Font(this, &SOutlinerItemViewBase::GetLabelFont)
		.Text(this, &SOutlinerItemViewBase::GetLabel)
		.ToolTipText(this, &SOutlinerItemViewBase::GetLabelToolTipText)
		.ColorAndOpacity(this, &SOutlinerItemViewBase::GetLabelColor)
		.Clipping(EWidgetClipping::ClipToBounds);
	}

	if (InArgs._AdditionalLabelContent.Widget != SNullWidget::NullWidget)
	{
		LabelContent = SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0.f, 0.f, 5.f, 0.f))
		[
			LabelContent.ToSharedRef()
		]

		+ SHorizontalBox::Slot()
		[
			InArgs._AdditionalLabelContent.Widget
		];

		LabelContent->SetClipping(EWidgetClipping::ClipToBounds);
	}

	SetForegroundColor(MakeAttributeSP(this, &SOutlinerItemViewBase::GetForegroundBasedOnSelection));

	static float IndentAmount = 10.f;

	TSharedRef<SWidget>	FinalWidget = 
			SNew( SHorizontalBox )

			+ SHorizontalBox::Slot()
			[
				SNew(SBox)
				.HeightOverride(this, &SOutlinerItemViewBase::GetHeight)
				[
					SNew( SHorizontalBox )

					// Expand track lanes button
					+ SHorizontalBox::Slot()
					.Padding(FMargin(2.f, 0.f, 2.f, 0.f))
					.VAlign( VAlign_Center )
					.AutoWidth()
					[
						SNew(SExpanderArrow, InTableRow)
						.IndentAmount(IndentAmount)
					]

					+ SHorizontalBox::Slot()
					.Padding( InnerNodePadding )
					[
						SNew( SBorder )
						.BorderImage( FAppStyle::GetBrush( "Sequencer.AnimationOutliner.TopLevelBorder_Collapsed" ) )
						.BorderBackgroundColor( this, &SOutlinerItemViewBase::GetNodeInnerBackgroundTint )
						.Padding( FMargin(0) )
						[
							SNew( SHorizontalBox )
							
							// Icon
							+ SHorizontalBox::Slot()
							.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(SOverlay)

								+ SOverlay::Slot()
								[
									SNew(SImage)
									.Image(this, &SOutlinerItemViewBase::GetIconBrush)
									.ColorAndOpacity(this, &SOutlinerItemViewBase::GetIconTint)
								]

								+ SOverlay::Slot()
								.VAlign(VAlign_Top)
								.HAlign(HAlign_Right)
								[
									SNew(SImage)
									.Image(this, &SOutlinerItemViewBase::GetIconOverlayBrush)
								]

								+ SOverlay::Slot()
								[
									SNew(SSpacer)
									.Visibility(EVisibility::Visible)
									.ToolTipText(this, &SOutlinerItemViewBase::GetIconToolTipText)
								]
							]

							// Label Slot
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
							[
								LabelContent.ToSharedRef()
							]

							// Arbitrary customization slot
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								InArgs._CustomContent.Widget
							]
						]
					]
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				InArgs._RightGutterContent.Widget
			];

	ChildSlot
	[
		FinalWidget
	];
}

FText SOutlinerItemViewBase::GetLabel() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	return Outliner ? Outliner->GetLabel() : FText();
}

FSlateColor SOutlinerItemViewBase::GetLabelColor() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	return Outliner ? Outliner->GetLabelColor() : (IsDimmed() ? FSlateColor::UseSubduedForeground() : FSlateColor::UseForeground());
}

FSlateFontInfo SOutlinerItemViewBase::GetLabelFont() const
{
	if (TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin())
	{
		return Outliner->GetLabelFont();
	}
	return FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont");
}

FText SOutlinerItemViewBase::GetLabelToolTipText() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	return Outliner ? Outliner->GetLabelToolTipText() : FText();
}

bool SOutlinerItemViewBase::IsRenameValid(const FText& NewName, FText& OutErrorMessage) const
{
	// By default we defer to the model
	TViewModelPtr<IRenameableExtension> Renamable = WeakRenameExtension.Pin();
	if (Renamable)
	{
		return Renamable->IsRenameValid(NewName, OutErrorMessage);
	}
	
	OutErrorMessage = LOCTEXT("NotRenameable", "Item is not renamable");
	return false;
}

void SOutlinerItemViewBase::OnNodeLabelTextCommitted(const FText& NewLabel, ETextCommit::Type CommitType)
{
	TViewModelPtr<IRenameableExtension> Renamable = WeakRenameExtension.Pin();
	if (Renamable)
	{
		Renamable->Rename(NewLabel);
	}
}

const FSlateBrush* SOutlinerItemViewBase::GetIconBrush() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	return Outliner ? Outliner->GetIconBrush() : nullptr;
}

FSlateColor SOutlinerItemViewBase::GetIconTint() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	return Outliner ? Outliner->GetIconTint() : GetLabelColor();
}

const FSlateBrush* SOutlinerItemViewBase::GetIconOverlayBrush() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	return Outliner ? Outliner->GetIconOverlayBrush() : nullptr;
}

FText SOutlinerItemViewBase::GetIconToolTipText() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	return Outliner ? Outliner->GetIconToolTipText() : FText();
}

bool SOutlinerItemViewBase::IsNodeLabelReadOnly() const
{
	const bool bIsReadOnly = IsReadOnlyAttribute.Get();
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	TViewModelPtr<IRenameableExtension> Renamable = WeakRenameExtension.Pin();

	return bIsReadOnly || !Editor || Editor->IsReadOnly() || !Renamable || !Renamable->CanRename();
}

bool SOutlinerItemViewBase::IsDimmed() const
{
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	if (!Outliner || !Editor)
	{
		return true;
	}

	if (Editor->IsReadOnly())
	{
		return true;
	}

	// If the item implements IDimmableExtension, trust that it will do everything
	// to figure out whether we should dim it or not.
	TViewModelPtr<IDimmableExtension> Dimmable = Outliner.ImplicitCast();
	if (Dimmable)
	{
		return Dimmable->IsDimmed();
	}

	const TViewModelPtr<IDeactivatableExtension> Deactivatable = Outliner.ImplicitCast();
	if (Deactivatable && Deactivatable->IsDeactivated())
	{
		return true;
	}

	// Otherwise, provide some simple default logic: dim the item if it's muted.
	TViewModelPtr<IMutableExtension> Mutable = Outliner.ImplicitCast();
	if (Mutable && Mutable->IsMuted())
	{
		return true;
	}

	return false;
}

FOptionalSize SOutlinerItemViewBase::GetHeight() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();

	return Outliner
		? Outliner->GetOutlinerSizing().GetTotalHeight()
		: 10.f;
}

const FSlateBrush* SOutlinerItemViewBase::GetNodeBorderImage() const
{
	TSharedPtr<FViewModel> DataModel = WeakOutlinerExtension.Pin().AsModel();
	const bool bHasChildren = DataModel.IsValid() && (bool)DataModel->GetDescendantsOfType<IOutlinerExtension>();

	return bHasChildren ? ExpandedBackgroundBrush : CollapsedBackgroundBrush;
}

FSlateColor SOutlinerItemViewBase::GetNodeInnerBackgroundTint() const
{
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();
	if (OutlinerItem && ItemStyle == EOutlinerItemViewBaseStyle::InsideContainer)
	{
		EOutlinerSelectionState SelectionState = OutlinerItem->GetSelectionState();
		if (EnumHasAnyFlags(SelectionState, EOutlinerSelectionState::SelectedDirectly))
		{
			return FStyleColors::Select;
		}
		else if (EnumHasAnyFlags(SelectionState, EOutlinerSelectionState::HasSelectedKeys | EOutlinerSelectionState::HasSelectedTrackAreaItems))
		{
			return FStyleColors::Header;
		}
		else if (Editor->GetOutliner()->GetHoveredItem() == OutlinerItem)
		{
			return FLinearColor( FColor( 52, 52, 52, 255 ) );
		}
		else
		{
			return FLinearColor( FColor( 48, 48, 48, 255 ) );
		}
	}
	else
	{
		return FLinearColor( 0.f, 0.f, 0.f, 0.f );
	}
}

FSlateColor SOutlinerItemViewBase::GetForegroundBasedOnSelection() const
{
	return IsRowSelectedAttribute.Get() ? TableRowStyle->SelectedTextColor : TableRowStyle->TextColor;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

