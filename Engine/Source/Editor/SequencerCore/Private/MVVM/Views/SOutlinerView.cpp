// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Views/SOutlinerViewRow.h"

#include "Algo/BinarySearch.h"
#include "Algo/Find.h"
#include "Containers/ArrayView.h"
#include "Framework/Layout/Overscroll.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/IDroppableExtension.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypesPrivate.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Views/STrackLane.h"
#include "MVVM/Views/TreeViewTraits.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "ScopedTransaction.h"
#include "SequencerCoreFwd.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateBrush.h"
#include "Styling/WidgetStyle.h"
#include "Templates/Tuple.h"
#include "Types/SlateStructs.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

namespace UE::Sequencer
{

SOutlinerView::SOutlinerView()
{}

SOutlinerView::~SOutlinerView()
{
	TrackArea.Reset();
	PinnedTreeViews.Empty();
}

void SOutlinerView::Construct(const FArguments& InArgs, TWeakPtr<FOutlinerViewModel> InWeakOutliner, const TSharedRef<STrackAreaView>& InTrackArea)
{
	WeakOutliner = InWeakOutliner;
	TrackArea = InTrackArea;
	bUpdatingTreeSelection = false;
	bRightMouseButtonDown = false;
	bShowPinnedNodes = false;
	bSelectionChangesPending = false;
	bRefreshPhysicalGeometry = false;

	Selection = InArgs._Selection;

	if (Selection)
	{
		TSharedPtr<FOutlinerSelection> OutlinerSelection = Selection->GetOutlinerSelection();
		if (OutlinerSelection)
		{
			OutlinerSelection->OnChanged.AddSP(this, &SOutlinerView::UpdateViewSelectionFromModel);
		}
	}

	ColumnMetaData = MakeShared<FOutlinerHeaderRowWidgetMetaData>();

	HeaderRow = SNew(SHeaderRow)
	.Visibility(EVisibility::Collapsed)
	.AddMetaData(ColumnMetaData.ToSharedRef());

	UpdateOutlinerColumns();

	WeakOutliner.Pin()->OnRefreshed.AddSP(this, &SOutlinerView::Refresh);

	STreeView::Construct
	(
		STreeView::FArguments()
		.TreeItemsSource(&RootNodes)
		.OnItemToString_Debug(this, &SOutlinerView::OnItemToString_Debug)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SOutlinerView::OnGenerateRow)
		.OnGetChildren(this, &SOutlinerView::OnGetChildren)
		.HeaderRow(HeaderRow)
		.ExternalScrollbar(InArgs._ExternalScrollbar)
		.OnExpansionChanged(this, &SOutlinerView::OnExpansionChanged)
		.OnTreeViewScrolled(this, &SOutlinerView::HandleTableViewScrolled)
		.AllowOverscroll(EAllowOverscroll::No)
		.OnContextMenuOpening( this, &SOutlinerView::OnContextMenuOpening )
		.OnSetExpansionRecursive(this, &SOutlinerView::SetItemExpansionRecursive)
		.HighlightParentNodesForSelection(true)
		.AllowInvisibleItemSelection(true)  //without this we deselect everything when we filter or we collapse, etc..
	);

	UpdateViewSelectionFromModel();
}

void SOutlinerView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const bool bWasPendingRefresh = IsPendingRefresh();

	STreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (!FSlateApplication::Get().AnyMenusVisible())
	{
		DelayedEventSuppressor = nullptr;
	}

	if (bWasPendingRefresh || bRefreshPhysicalGeometry)
	{
		UpdatePhysicalGeometry(true);
		bRefreshPhysicalGeometry = false;
	}

	if (bWasPendingRefresh && TrackArea)
	{
		// Rows retired by the generation pass that just ran in STreeView::Tick may
		// have dropped the last strong reference to their track lanes. Remove the
		// resulting empty slots from the track area so they don't accumulate.
		TrackArea->PurgeDeadTrackSlots();
	}
}

TSharedPtr<FOutlinerViewModel> SOutlinerView::GetOutlinerModel() const
{
	return WeakOutliner.Pin();
}

FString SOutlinerView::OnItemToString_Debug(TWeakViewModelPtr<IOutlinerExtension> InWeakModel)
{
	TViewModelPtr<IOutlinerExtension> Model = InWeakModel.ImplicitPin();
	return Model ? Model->GetLabel().ToString() : FString();
}

int32 SOutlinerView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	return STreeView::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FReply SOutlinerView::OnDragRow(const FGeometry&, const FPointerEvent&, TSharedRef<SOutlinerViewRow> InRow)
{
	TSharedPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		return FReply::Unhandled();
	}

	TArray<TWeakViewModelPtr<IOutlinerExtension>> WeakSelectedItems = GetSelectedItems();
	if (WeakSelectedItems.Num() > 0)
	{
		for (int32 Index = WeakSelectedItems.Num()-1; Index >= 0; --Index)
		{
			TSharedPtr<IDraggableOutlinerExtension> Draggable = WeakSelectedItems[Index].ImplicitPin();
			if (!Draggable || !Draggable->CanDrag())
			{
				// Order is not important so we can opt for performance with RemoveAtSwap
				WeakSelectedItems.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}

		// If there were no nodes selected, don't start a drag drop operation.
		if (WeakSelectedItems.Num() == 0)
		{
			return FReply::Unhandled();
		}

		TSharedRef<FDragDropOperation> DragDropOp = Outliner->InitiateDrag( MoveTemp(WeakSelectedItems) );
		return FReply::Handled().BeginDragDrop( DragDropOp );
	}

	return FReply::Unhandled();
}

void SOutlinerView::ReportChildRowGeometry(const TViewModelPtr<IOutlinerExtension>& InNode, const FGeometry& InGeometry)
{
	FCachedGeometry* PhysicalNode = PhysicalNodes.FindByPredicate(
			[InNode](FCachedGeometry& Item) { return Item.WeakItem.Pin() == InNode; });
	if (PhysicalNode)
	{
		PhysicalNode->PhysicalHeight = InGeometry.Size.Y;
	}
}

void SOutlinerView::GetVisibleItems(TArray<TViewModelPtr<IOutlinerExtension>>& OutItems) const
{
	OutItems.Reserve(OutItems.Num() + PhysicalNodes.Num());
	for (const FCachedGeometry& CachedGeometry : PhysicalNodes)
	{
		if (TViewModelPtr<IOutlinerExtension> Item = CachedGeometry.WeakItem.Pin())
		{
			OutItems.Add(Item);
		}
	}
}

void SOutlinerView::ForceSetSelectedItems(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InItems)
{
	Private_ClearSelection();
	for (const TWeakViewModelPtr<IOutlinerExtension>& Item : InItems)
	{
		Private_SetItemSelection(Item, true, false);
	}

	Private_SignalSelectionChanged(ESelectInfo::Direct);
}

TViewModelPtr<IOutlinerExtension> SOutlinerView::HitTestNode(float InPhysical) const
{
	// Find the first node with a top after the specified value - the hit node must be the one preceeding this
	const int32 FoundIndex = Algo::UpperBoundBy(PhysicalNodes, InPhysical, &FCachedGeometry::PhysicalTop) - 1;
	if (FoundIndex >= 0)
	{
		return PhysicalNodes[FoundIndex].WeakItem.Pin();
	}

	return nullptr;
}

float SOutlinerView::PhysicalToVirtual(float InPhysical) const
{
	// Find the first node with a top after the specified value - the hit node must be the one preceeding this
	const int32 FoundIndex = Algo::UpperBoundBy(PhysicalNodes, InPhysical, &FCachedGeometry::PhysicalTop) - 1;
	if (FoundIndex >= 0)
	{
		FCachedGeometry Found = PhysicalNodes[FoundIndex];
		const float FractionalHeight = (Found.PhysicalHeight != 0.f) ? (InPhysical - Found.PhysicalTop) / Found.PhysicalHeight : 0.f;
		return Found.VirtualTop + Found.VirtualHeight * FractionalHeight;
	}

	if (PhysicalNodes.Num())
	{
		const FCachedGeometry& First = PhysicalNodes[0];
		if (InPhysical < First.PhysicalTop)
		{
			return First.VirtualTop + (InPhysical - First.PhysicalTop);
		}
		else
		{
			const FCachedGeometry& Last = PhysicalNodes.Last();
			return First.VirtualTop + (InPhysical - Last.PhysicalTop);
		}
	}

	return InPhysical;
}

float SOutlinerView::VirtualToPhysical(float InVirtual) const
{
	// Find the first node with a top after the specified value - the hit node must be the one preceeding this
	int32 FoundIndex = Algo::UpperBoundBy(PhysicalNodes, InVirtual, &FCachedGeometry::VirtualTop) - 1;

	if (FoundIndex >= 0)
	{
		const FCachedGeometry& Found = PhysicalNodes[FoundIndex];
		const float FractionalHeight = (Found.VirtualHeight != 0.f) ? (InVirtual - Found.VirtualTop) / Found.VirtualHeight : 0.f;
		return Found.PhysicalTop + Found.PhysicalHeight * FractionalHeight;
	}

	if (PhysicalNodes.Num())
	{
		const FCachedGeometry& Last = PhysicalNodes.Last();
		return Last.PhysicalTop + (InVirtual - Last.VirtualTop);
	}

	return InVirtual;
}

int32 SOutlinerView::CreateOutlinerColumnsForGroup(int32 ColumnIndex, EOutlinerColumnGroup Group)
{
	const int32 NumColumns = OutlinerColumns.Num();

	int32 NumAdded = 0;
	for ( ; ColumnIndex < NumColumns; ++ColumnIndex, ++NumAdded)
	{
		const TSharedPtr<IOutlinerColumn>& Column = OutlinerColumns[ColumnIndex];

		FOutlinerColumnPosition Position = Column->GetPosition();
		if (Position.Group != Group)
		{
			// OutlinerColumns is sorted so if we encountered the wrong group we should stop immediately
			break;
		}

		FName                 ColumnName = Column->GetColumnName();
		FOutlinerColumnLayout Layout     = Column->GetLayout();

		// Add the meta-data required for our custom row panel
		ColumnMetaData->Columns.Add(Layout);

		// Keep track of how to create widgets for this column
		// Column Generators must be defined before HeaderRow->AddColumn since AddColumn might re-generate widgets
		ColumnGenerators.Add(ColumnName, [Column, ColumnName](const FCreateOutlinerColumnParams& Params, const TSharedRef<SOutlinerViewRow>& Row){
			return (Params.OutlinerExtension.IsValid() && Params.OutlinerExtension->SupportsOutlinerColumn(ColumnName) && Column->IsItemCompatibleWithColumn(Params))
				? Column->CreateColumnWidget(Params, Row)
				: nullptr;
		});

		// Add the column itself to the header row
		if (Layout.SizeMode == EOutlinerColumnSizeMode::Fixed)
		{
			HeaderRow->AddColumn(
				SHeaderRow::Column(ColumnName)
				.FixedWidth(Layout.Width)
			);
		}
		else
		{
			HeaderRow->AddColumn(
				SHeaderRow::Column(ColumnName)
				.FillWidth(Layout.Width)
			);
		}
	}

	return NumAdded;
}

void SOutlinerView::InsertSeparatorColumn(int32 InsertIndex, int32 SeparatorID)
{
	static const FName NAME_Separator("Separator");

	FOutlinerColumnLayout SeparatorLayout{
		1.f, /* Width */
		FMargin(0.f),
		HAlign_Fill,
		VAlign_Fill,
		EOutlinerColumnSizeMode::Fixed,
		EOutlinerColumnFlags::Hidden
	};

	FName SeparatorName(NAME_Separator, SeparatorID);

	// Add a 1px separator column
	ColumnMetaData->Columns.Insert(SeparatorLayout, InsertIndex);

	// Column Generators must be defined before HeaderRow->InsertColumn since InsertColumn might re-generate widgets
	ColumnGenerators.Add(SeparatorName,
		[](const FCreateOutlinerColumnParams&, const TSharedRef<SOutlinerViewRow>&)->TSharedPtr<SWidget>
	{
		return SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Sequencer.Outliner.Separator"));
	}
	);

	HeaderRow->InsertColumn(
		SHeaderRow::Column(SeparatorName)
		.FixedWidth(1.f),
		InsertIndex
	);
}

void SOutlinerView::UpdateOutlinerColumns()
{
	// Sort the columns by position
	Algo::SortBy(OutlinerColumns, &IOutlinerColumn::GetPosition);

	// Clear columns to ensure consistent order when building UI from Map
	HeaderRow->ClearColumns();
	ColumnMetaData->Columns.Empty();
	ColumnGenerators.Empty();

	// ----------------------------------------------------------------------------------------------------------
	// Populate columns
	int32 CurrentIndex = 0;

	const int32 NumLeftGutter = CreateOutlinerColumnsForGroup(CurrentIndex, EOutlinerColumnGroup::LeftGutter);
	CurrentIndex += NumLeftGutter;
	
	const int32 NumToggles = CreateOutlinerColumnsForGroup(CurrentIndex, EOutlinerColumnGroup::AssetToggles);
	CurrentIndex += NumToggles;

	const int32 NumMuteSoloToggles = CreateOutlinerColumnsForGroup(CurrentIndex, EOutlinerColumnGroup::TransientToggles);
	CurrentIndex += NumMuteSoloToggles;

	const int32 NumCenter = CreateOutlinerColumnsForGroup(CurrentIndex, EOutlinerColumnGroup::Center);
	CurrentIndex += NumCenter;

	const int32 NumRightGutter = CreateOutlinerColumnsForGroup(CurrentIndex, EOutlinerColumnGroup::RightGutter);
	CurrentIndex += NumRightGutter;

	// ----------------------------------------------------------------------------------------------------------
	// Add some padding to the leading and trailing edge of the first and last columns in each group respectively
	//    This is implemented this way because columns can be turned on and off dynamically, but we must
	//    always have a consistent padding within the group. Separators dynamically appear based on the presence of
	//    each group so we can't put padding on those
	const float DefaultGroupPadding = 2.f;
	if (NumToggles > 0)
	{
		const int32 FirstIndex = NumLeftGutter;
		const int32 LastIndex = FirstIndex + NumToggles - 1;

		ColumnMetaData->Columns[FirstIndex].CellPadding.Left += DefaultGroupPadding;
		ColumnMetaData->Columns[LastIndex].CellPadding.Right += DefaultGroupPadding;
	}

	if (NumMuteSoloToggles > 0)
	{
		const int32 FirstIndex = NumLeftGutter + NumToggles;
		const int32 LastIndex = FirstIndex + NumMuteSoloToggles - 1;

		ColumnMetaData->Columns[FirstIndex].CellPadding.Left += DefaultGroupPadding;
		ColumnMetaData->Columns[LastIndex].CellPadding.Right += DefaultGroupPadding;
	}

	if (NumCenter > 0)
	{
		const int32 FirstIndex = NumLeftGutter + NumToggles + NumMuteSoloToggles;
		const int32 LastIndex = FirstIndex + NumCenter - 1;

		ColumnMetaData->Columns[FirstIndex].CellPadding.Left += DefaultGroupPadding;
		ColumnMetaData->Columns[LastIndex].CellPadding.Right += DefaultGroupPadding;
	}

	// No additional padding on the right gutter intentionally

	// ----------------------------------------------------------------------------------------------------------
	// Add separators between the groups.
	//      Only add separators if they actually separate columns.
	int32 InsertIndex = NumLeftGutter;
	if (InsertIndex < ColumnMetaData->Columns.Num())
	{
		int32 NumSeparators = 0;

		InsertSeparatorColumn(InsertIndex++, ++NumSeparators);

		auto InsertSeparatorAfterGroup = [this, &NumSeparators, &InsertIndex](const int32 InIndex)
		{
			InsertIndex += InIndex;
			if (InsertIndex < ColumnMetaData->Columns.Num())
			{
				InsertSeparatorColumn(InsertIndex++, ++NumSeparators);
			}
		};

		if (NumToggles > 0)
        {
        	InsertSeparatorAfterGroup(NumToggles);
        }

		if (NumMuteSoloToggles > 0)
		{
			InsertSeparatorAfterGroup(NumMuteSoloToggles);
		}

		if (NumCenter > 0)
		{
			InsertSeparatorAfterGroup(NumCenter);
		}
	}
}

void SOutlinerView::AddPinnedTreeView(TSharedPtr<SOutlinerView> PinnedTreeView)
{
	PinnedTreeViews.Add(PinnedTreeView);
	PinnedTreeView->SetPrimaryTreeView(SharedThis(this));
}

void SOutlinerView::SetOutlinerColumns(const TArray<TSharedPtr<IOutlinerColumn>>& InOutlinerColumns)
{
	// Reset the way rows are constructed with an updated list of Outliner Columns
	OutlinerColumns = InOutlinerColumns;
	UpdateOutlinerColumns();
}

void SOutlinerView::OnRightMouseButtonDown(const FPointerEvent& MouseEvent)
{
	STreeView::OnRightMouseButtonDown(MouseEvent);
	bRightMouseButtonDown = true;
}

void SOutlinerView::OnRightMouseButtonUp(const FPointerEvent& MouseEvent)
{
	STreeView::OnRightMouseButtonUp(MouseEvent);
	bRightMouseButtonDown = false;
}

FReply SOutlinerView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FReply Result = STreeView::OnDragOver(MyGeometry, DragDropEvent);

	if (!Result.IsEventHandled())
	{
		TSharedPtr<FOutlinerViewModelDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FOutlinerViewModelDragDropOp>();
		if (DragDropOp.IsValid())
		{
			// Reset tooltip to indicate that the dragged objects can be moved to the root (ie. unparented)
			DragDropOp->ResetToDefaultToolTip();
		}

		return Result.Handled();
	}
	return Result;
}

FReply SOutlinerView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		return FReply::Handled();
	}

	FReply Result = STreeView::OnDrop(MyGeometry, DragDropEvent);

	TSharedPtr<FOutlinerViewModelDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FOutlinerViewModelDragDropOp>();
	if (!Result.IsEventHandled() && DragDropOp.IsValid() && DragDropOp->GetDraggedViewModels().Num())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackNode", "MoveItems", "Move items."));

		TViewModelPtr<IDroppableExtension> Root = Outliner->GetRootItem().ImplicitCast();
		if (Root)
		{
			Root->ProcessDragOperation(*DragDropOp);
		}

		return FReply::Handled();
	}
	
	return Result;
}

FReply SOutlinerView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const TArrayView<const TWeakViewModelPtr<IOutlinerExtension>> ItemsSourceRef = SListView<TWeakViewModelPtr<IOutlinerExtension>>::GetItems();

	auto IsSelectable = [](const TWeakViewModelPtr<IOutlinerExtension>& WeakModel)
	{
		TSharedPtr<ISelectableExtension> Selectable = WeakModel.ImplicitPin();
		return !Selectable || EnumHasAnyFlags(Selectable->IsSelectable(), ESelectionIntent::PersistentSelection);
	};

	// Don't respond to key-presses containing "Alt" as a modifier
	if (ItemsSourceRef.Num() > 0 && !InKeyEvent.IsAltDown())
	{
		bool bWasHandled = false;
		NullableItemType ItemNavigatedTo(nullptr);

		// Check for selection manipulation keys (Up, Down, Home, End, PageUp, PageDown)
		if (InKeyEvent.GetKey() == EKeys::Up)
		{
			int32 SelectionIndex = 0;
			if (TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::IsPtrValid(SelectorItem))
			{
				SelectionIndex = ItemsSourceRef.Find(TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::NullableItemTypeConvertToItemType(SelectorItem));
			}

			--SelectionIndex;

			for (; SelectionIndex >=0; --SelectionIndex)
			{
				if (IsSelectable(ItemsSourceRef[SelectionIndex]))
				{
					ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
					break;
				}
			}
			bWasHandled = true;
		}
		else if (InKeyEvent.GetKey() == EKeys::Down)
		{
			int32 SelectionIndex = 0;
			if (TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::IsPtrValid(SelectorItem))
			{
				SelectionIndex = ItemsSourceRef.Find(TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::NullableItemTypeConvertToItemType(SelectorItem));
			}

			++SelectionIndex;

			for (; SelectionIndex < ItemsSourceRef.Num(); ++SelectionIndex)
			{
				if (IsSelectable(ItemsSourceRef[SelectionIndex]))
				{
					ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
					break;
				}
			}
			bWasHandled = true;
		}
		else if (InKeyEvent.GetKey() == EKeys::Home)
		{
			// Select the first item
			for (int32 SelectionIndex = 0; SelectionIndex < ItemsSourceRef.Num(); ++SelectionIndex)
			{
				if (IsSelectable(ItemsSourceRef[SelectionIndex]))
				{
					ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
					break;
				}
			}
			bWasHandled = true;
		}
		else if (InKeyEvent.GetKey() == EKeys::End)
		{
			// Select the last item
			for (int32 SelectionIndex = ItemsSourceRef.Num() -1; SelectionIndex >=0 ; --SelectionIndex)
			{
				if (IsSelectable(ItemsSourceRef[SelectionIndex]))
				{
					ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
					break;
				}
			}
			bWasHandled = true;
		}
		else if (InKeyEvent.GetKey() == EKeys::PageUp)
		{
			int32 SelectionIndex = 0;
			if (TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::IsPtrValid(SelectorItem))
			{
				SelectionIndex = ItemsSourceRef.Find(TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::NullableItemTypeConvertToItemType(SelectorItem));
			}

			int32 NumItemsInAPage = GetNumLiveWidgets();
			int32 Remainder = NumItemsInAPage % GetNumItemsPerLine();
			NumItemsInAPage -= Remainder;

			if (SelectionIndex >= NumItemsInAPage)
			{
				// Select an item on the previous page
				SelectionIndex = SelectionIndex - NumItemsInAPage;

				// Scan up for the first selectable node
				for (; SelectionIndex >= 0; --SelectionIndex)
				{
					if (IsSelectable(ItemsSourceRef[SelectionIndex]))
					{
						ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
						break;
					}
				}
			}

			// If we had less than a page to jump, or we haven't found a selectable node yet,
			// scan back toward our current node until we find one.
			if (!ItemNavigatedTo.Pin())
			{
				SelectionIndex = 0;
				for (; SelectionIndex < ItemsSourceRef.Num(); ++SelectionIndex)
				{
					if (IsSelectable(ItemsSourceRef[SelectionIndex]))
					{
						ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
						break;
					}
				}
			}

			bWasHandled = true;
		}
		else if (InKeyEvent.GetKey() == EKeys::PageDown)
		{
			int32 SelectionIndex = 0;
			if (TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::IsPtrValid(SelectorItem))
			{
				SelectionIndex = ItemsSourceRef.Find(TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::NullableItemTypeConvertToItemType(SelectorItem));
			}

			int32 NumItemsInAPage = GetNumLiveWidgets();
			int32 Remainder = NumItemsInAPage % GetNumItemsPerLine();
			NumItemsInAPage -= Remainder;


			if (SelectionIndex < ItemsSourceRef.Num() - NumItemsInAPage)
			{
				// Select an item on the next page
				SelectionIndex = SelectionIndex + NumItemsInAPage;

				for (; SelectionIndex < ItemsSourceRef.Num(); ++SelectionIndex)
				{
					if (IsSelectable(ItemsSourceRef[SelectionIndex]))
					{
						ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
						break;
					}
				}
			}

			// If we had less than a page to jump, or we haven't found a selectable node yet,
			// scan back toward our current node until we find one.
			if (!ItemNavigatedTo.Pin())
			{
				SelectionIndex = ItemsSourceRef.Num() - 1;
				for (; SelectionIndex >= 0; --SelectionIndex)
				{
					if (IsSelectable(ItemsSourceRef[SelectionIndex]))
					{
						ItemNavigatedTo = ItemsSourceRef[SelectionIndex];
						break;
					}
				}
			}
			bWasHandled = true;
		}
		else if (InKeyEvent.GetKey() == EKeys::SpaceBar)
		{
			// SListView behavior is: Change selected status of item. Bypass that here. We don't want that, but instead want spacebar to go to SequencerCommands (toggle play)
			return FReply::Unhandled();
		}

		if (TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::IsPtrValid(ItemNavigatedTo))
		{
			TWeakViewModelPtr<IOutlinerExtension> ItemToSelect(TListTypeTraits<TWeakViewModelPtr<IOutlinerExtension>>::NullableItemTypeConvertToItemType(ItemNavigatedTo));
			NavigationSelect(ItemToSelect, InKeyEvent);
		}

		if (bWasHandled)
		{
			return FReply::Handled();
		}
	}

	return STreeView<TWeakViewModelPtr<IOutlinerExtension>>::OnKeyDown(MyGeometry, InKeyEvent);
}

void SOutlinerView::Private_UpdateParentHighlights()
{
	this->ClearHighlightedItems();

	// For the Outliner, we want to highlight parent items even if the current selection is not visible (i.e collapsed)
	for (TWeakViewModelPtr<IOutlinerExtension> WeakSelectedItem : GetSelectedItems())
	{
		TViewModelPtr<IOutlinerExtension> SelectedItem = WeakSelectedItem.Pin();
		if (SelectedItem.IsValid())
		{
			for (TViewModelPtr<IOutlinerExtension> Parent : SelectedItem.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
			{
				Private_SetItemHighlighted(Parent, true);
			}
		}
	}
}

void SOutlinerView::Private_SetItemSelection( TWeakViewModelPtr<IOutlinerExtension> TheItem, bool bShouldBeSelected, bool bWasUserDirected )
{
	if (TSharedPtr<FOutlinerSpacer> Spacer = TheItem.ImplicitPin())
	{
		return;
	}

	STreeView::Private_SetItemSelection( TheItem, bShouldBeSelected, bWasUserDirected );
}

void SOutlinerView::Private_ClearSelection()
{
	STreeView::Private_ClearSelection();
}

void SOutlinerView::Private_SelectRangeFromCurrentTo( TWeakViewModelPtr<IOutlinerExtension> InRangeSelectionEnd )
{
	STreeView::Private_SelectRangeFromCurrentTo(InRangeSelectionEnd);
}

void SOutlinerView::Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo)
{
	STreeView::Private_SignalSelectionChanged(SelectInfo);

	UpdateModelSelectionFromView();
}

void SOutlinerView::UpdateViewSelectionFromModel()
{
	if (!Selection)
	{
		return;
	}

	TSharedPtr<const FOutlinerSelection> OutlinerSelection = Selection->GetOutlinerSelection();
	if (OutlinerSelection != nullptr)
	{
		Private_ClearSelection();

		for (TViewModelPtr<IOutlinerExtension> SelectedItem : *OutlinerSelection)
		{
			Private_SetItemSelection(SelectedItem, true, false);
		}

		Private_SignalSelectionChanged(ESelectInfo::Direct);
	}
}

void SOutlinerView::UpdateModelSelectionFromView()
{
	if (!Selection || Selection->IsTriggeringSelectionChangedEvents())
	{
		return;
	}

	TSharedPtr<FOutlinerSelection> OutlinerSelection = Selection->GetOutlinerSelection();
	if (OutlinerSelection == nullptr)
	{
		return;
	}

	DelayedEventSuppressor = Selection->SuppressEventsLongRunning();

	OutlinerSelection->Empty();
	for (TWeakViewModelPtr<IOutlinerExtension> SelectedItem : GetSelectedItems())
	{
		OutlinerSelection->Select(SelectedItem);
	}

	if (bRightMouseButtonDown == false)
	{
		// If we're not going to open a context menu - trigger events now
		DelayedEventSuppressor = nullptr;
	}
}

TSharedPtr<SWidget> SOutlinerView::OnContextMenuOpening()
{
	TSharedPtr<SWidget> ContextMenu;

	// Open a context menu for the first selected item if it is selectable
	if (auto It = GetSelectedItems(); It.Num() > 0) 
	{
		TWeakViewModelPtr<IOutlinerExtension> WeakSelectedItem = It[0];
		TViewModelPtr<IOutlinerExtension> SelectedItem = WeakSelectedItem.Pin();

		// Items are selectable by default, but can opt-in to conditionally selectable
		// TODO: This should be checked in the selection handler?
		TViewModelPtr<ISelectableExtension> Selectable = WeakSelectedItem.ImplicitPin();
		const bool bIsSelectable = !Selectable || EnumHasAnyFlags(Selectable->IsSelectable(), ESelectionIntent::ContextMenu);

		if (bIsSelectable)
		{
			TSharedPtr<FEditorViewModel> EditorViewModel = GetOutlinerModel()->GetEditor();
			ContextMenu = SelectedItem->CreateContextMenuWidget(EditorViewModel);
		}
	}

	// Otherwise, add a general menu for options
	TSharedPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (Outliner && !ContextMenu)
	{
		ContextMenu = Outliner->CreateContextMenuWidget();
	}

	return ContextMenu;
}

void SOutlinerView::Refresh()
{
	TSharedPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();

	RootNodes.Reset();
	if (!Outliner)
	{
		return;
	}

	// Capture before GetTopLevelItems() consumes the flag
	const bool bHierarchyChanged = Outliner->IsHierarchyInvalidated();

	for (TWeakViewModelPtr<IOutlinerExtension> WeakItem : Outliner->GetTopLevelItems())
	{
		TViewModelPtr<IOutlinerExtension> Extension = WeakItem.Pin();
		if (!Extension || Extension->IsFilteredOut())
		{
			continue;
		}

		// Only add pinned nodes if this is showing pinned only
		bool bIsPinned = Extension->ShouldAnchorToTop();

		constexpr bool bIncludeThis = true;
		if (!bIsPinned)
		{
			for (const TViewModelPtr<IPinnableExtension>& Pinnable : Extension.AsModel()->GetAncestorsOfType<IPinnableExtension>(bIncludeThis))
			{
				if (Pinnable->IsPinned())
				{
					bIsPinned = true;
					break;
				}
			}
		}

		if (bIsPinned == bShowPinnedNodes)
		{
			RootNodes.Add(Extension);
		}
	}

	// Reset item expansion since we don't know if any expansion states may have changed in-between refreshes
	{
		STreeView::OnExpansionChanged.Unbind();

		ClearExpandedItems();

		FViewModelPtr RootItem = Outliner->GetRootItem();
		if (RootItem)
		{
			for (TViewModelPtr<IOutlinerExtension> Child : RootItem->GetDescendantsOfType<IOutlinerExtension>())
			{
				SetItemExpansion(Child, Child->IsExpanded());
			}
		}

		STreeView::OnExpansionChanged.BindSP(this, &SOutlinerView::OnExpansionChanged);
	}

	if (bHierarchyChanged)
	{
		// Hierarchy changed (tracks/bindings added or removed): discard all cached row widgets
		// so that SOutlinerViewRowPanel::UpdateCells re-runs compatibility checks and generates
		// fresh column widgets for every row. Without this, newly compatible columns (e.g. the
		// AutoBake icon after adding a Control Rig) stay invisible until the user manually
		// triggers a full refresh (F5 / scroll).
		RebuildList();
	}
	else
	{
		// State-only change (mute, solo, deactivate, etc.): run an incremental generation pass
		// that reuses existing row widgets. This keeps section widgets, keys, decoration
		// backgrounds, and column key editors painted continuously without flickering.
		RequestListRefresh();
	}

	for (TSharedPtr<SOutlinerView> PinnedTreeView : PinnedTreeViews)
	{
		PinnedTreeView->Refresh();
	}
}

void SOutlinerView::ScrollByDelta(float DeltaInSlateUnits)
{
	ScrollBy( GetCachedGeometry(), DeltaInSlateUnits, EAllowOverscroll::No );
}

float SOutlinerView::GetVirtualTop() const
{
	return VirtualTop;
}

void SOutlinerView::HandleTableViewScrolled(double InScrollOffset)
{
	if (IsPendingRefresh())
	{
		bRefreshPhysicalGeometry = true;
	}
	else
	{
		UpdatePhysicalGeometry(false);
	}
}

void SOutlinerView::UpdatePhysicalGeometry(bool bIsRefresh)
{
	if (bIsRefresh)
	{
		// We need to first update virtual geometry in case the model hierarchy has changed, which is
		// very likely if we end up here from a tree view refresh.
		if (TSharedPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin())
		{
			if (FViewModelPtr RootItem = Outliner->GetRootItem())
			{
				IGeometryExtension::UpdateVirtualGeometry(0.f, RootItem);
			}
		}
	}

	const FChildren* Children = GetConstructedTableItems();
	const int32 NumSlots = Children->Num();

	PhysicalNodes.Empty();

	if (NumSlots == 0)
	{
		return;
	}

	const float FirstLineScrollOffset = GetFirstLineScrollOffset();
	float PhysicalOffset = 0.f;

	// Handle the first row: it might own a track lane, or reference a track lane owned by a parent
	// outliner item. In order to correctly position this track lane vertically, we need to accumulate
	// the item heights from whatever parents we have between the first row and the parent owns our
	// track lane.
	{
		TSharedRef<SOutlinerViewRow> FirstTreeViewRow  = StaticCastSharedRef<SOutlinerViewRow>(ConstCastSharedRef<SWidget>(Children->GetChildAt(0)));

		PhysicalOffset = -(FirstLineScrollOffset * FirstTreeViewRow->GetDesiredSize().Y);

		if (TSharedPtr<STrackLane> FirstRowTrackLane = FirstTreeViewRow->GetTrackLane(false))
		{
			FirstRowTrackLane->PositionParentTrackLanes(FirstTreeViewRow->GetDataModel(), PhysicalOffset);
		}
	}

	// Now we position any track lanes owned by the outliner items that are visible given the current scrolling.
	for (int32 Index = 0; Index < NumSlots; ++Index)
	{
		TSharedRef<const SOutlinerViewRow> TreeViewRow = StaticCastSharedRef<const SOutlinerViewRow>(Children->GetChildAt(Index));

		// Only get owned track lanes (no parent references... we don't want to move parent lanes down
		// to children items)
		if (TSharedPtr<STrackLane> TrackLane = TreeViewRow->GetTrackLane(true))
		{
			TrackLane->SetVerticalPosition(PhysicalOffset);
		}

		TViewModelPtr<IOutlinerExtension> DataModel = TreeViewRow->GetDataModel();
		if (DataModel)
		{
			TSharedPtr<IGeometryExtension> GeometryExtension = DataModel.ImplicitCast();

			const FVector2D TreeViewRowSize = TreeViewRow->GetDesiredSize();
			const float TreeViewRowHeight = TreeViewRowSize.Y > 0 ? TreeViewRowSize.Y : TreeViewRow->GetDesiredSize().Y;
			const FVirtualGeometry VirtualGeometry = GeometryExtension ? GeometryExtension->GetVirtualGeometry() : FVirtualGeometry();

			PhysicalNodes.Add(FCachedGeometry{
				DataModel,
				PhysicalOffset,
				TreeViewRowHeight,
				VirtualGeometry.GetTop(),
				VirtualGeometry.GetHeight(),
				VirtualGeometry.GetNestedHeight()
			});
		}

		PhysicalOffset += TreeViewRow->GetDesiredSize().Y;
	}
}

bool ShouldExpand(const TArrayView<const TWeakViewModelPtr<IOutlinerExtension>>& WeakSelectedItems, ETreeRecursion Recursion, TSharedPtr<FOutlinerViewModel> Outliner)
{
	bool bAllExpanded = true;
	for (TWeakViewModelPtr<IOutlinerExtension> WeakItem : WeakSelectedItems)
	{
		// @todo_sequencer_mvvm: Do we need to check selection state here if it's already in a supposed selection set??
		TViewModelPtr<IOutlinerExtension> SelectedItem = WeakItem.Pin();
		if (!SelectedItem || !SelectedItem->IsExpanded())
		{
			return true;
		}

		if (Recursion == ETreeRecursion::Recursive)
		{
			for (const TViewModelPtr<IOutlinerExtension>& Child : SelectedItem.AsModel()->GetChildrenOfType<IOutlinerExtension>())
			{
				FViewModelListIterator ChildChildren = Child.AsModel()->GetChildren(EViewModelListType::Outliner);
				if (ChildChildren)
				{
					if (!Child->IsExpanded())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

void SOutlinerView::ToggleExpandCollapseNodes(ETreeRecursion Recursion, bool bExpandAll, bool bCollapseAll)
{
	TSharedPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		return;
	}

	TArray<TWeakViewModelPtr<IOutlinerExtension>> WeakSelectedItems = GetSelectedItems();

	if (WeakSelectedItems.Num() > 0 && !bExpandAll && !bCollapseAll)
	{
		const bool bExpand = ShouldExpand(WeakSelectedItems, Recursion, Outliner);
		
		for (TWeakViewModelPtr<IOutlinerExtension> WeakItem : WeakSelectedItems)
		{
			TViewModelPtr<IOutlinerExtension> Item = WeakItem.Pin();
			if (Item)
			{
				ExpandCollapseNode(Item, bExpand, Recursion);
			}
		}
	}
	else if (SListView<TWeakViewModelPtr<IOutlinerExtension>>::GetItems().Num() > 0)
	{
		bool bExpand = ShouldExpand(SListView<TWeakViewModelPtr<IOutlinerExtension>>::GetItems(), Recursion, Outliner);

		if (bExpandAll)
		{
			bExpand = true;
		}
		if (bCollapseAll)
		{
			bExpand = false;
		}

		for (TWeakViewModelPtr<IOutlinerExtension> WeakItem : SListView<TWeakViewModelPtr<IOutlinerExtension>>::GetItems())
		{
			if (TViewModelPtr<IOutlinerExtension> Item = WeakItem.Pin())
			{
				ExpandCollapseNode(Item, bExpand, Recursion);
			}
		}
	}
}

void SOutlinerView::ExpandCollapseNode(TViewModelPtr<IOutlinerExtension> InViewModel, bool bExpansionState, ETreeRecursion Recursion)
{
	SetItemExpansion(InViewModel, bExpansionState);

	if (Recursion == ETreeRecursion::Recursive)
	{
		for (TViewModelPtr<IOutlinerExtension> Child : InViewModel.AsModel()->GetChildrenOfType<IOutlinerExtension>())
		{
			ExpandCollapseNode(Child, bExpansionState, ETreeRecursion::Recursive);
		}
	}
}

void SOutlinerView::OnExpansionChanged(TWeakViewModelPtr<IOutlinerExtension> InWeakItem, bool bIsExpanded)
{
	TViewModelPtr<IOutlinerExtension> SelectedItem = InWeakItem.Pin();
	if (!SelectedItem)
	{
		return;
	}

	SelectedItem->SetExpansion(bIsExpanded);

	// Expand any children that are also expanded
	for (const TViewModelPtr<IOutlinerExtension>& Child : SelectedItem.AsModel()->GetChildrenOfType<IOutlinerExtension>())
	{
		if (Child->IsExpanded())
		{
			SetItemExpansion(Child, true);
		}
	}

	// Also, expand/collapse any selected items if this item is selected
	TArray<TWeakViewModelPtr<IOutlinerExtension>> WeakSelectedItems = GetSelectedItems();
	if (WeakSelectedItems.Contains(InWeakItem))
	{
		for (TWeakViewModelPtr<IOutlinerExtension> WeakItem : WeakSelectedItems)
		{
			TViewModelPtr<IOutlinerExtension> Item = WeakItem.Pin();
			if (Item && Item != InWeakItem)
			{
				SetItemExpansion(Item, bIsExpanded);
			}
		}
	}
}

void SOutlinerView::SetItemExpansionRecursive(TWeakViewModelPtr<IOutlinerExtension> InItem, bool bIsExpanded)
{
	if (TViewModelPtr<IOutlinerExtension> Item = InItem.Pin())
	{
		ExpandCollapseNode(Item, bIsExpanded, ETreeRecursion::Recursive);
	}
}

void SOutlinerView::OnGetChildren(TWeakViewModelPtr<IOutlinerExtension> InParent, TArray<TWeakViewModelPtr<IOutlinerExtension>>& OutChildren) const
{
	TSharedPtr<FOutlinerViewModel> Outliner  = WeakOutliner.Pin();
	TViewModelPtr<IOutlinerExtension> DataModel = InParent.Pin();

	if (Outliner && DataModel)
	{
		for (TViewModelPtr<IOutlinerExtension> Child : DataModel.AsModel()->GetChildrenOfType<IOutlinerExtension>())
		{
			if (!Child->IsFilteredOut())
			{
				OutChildren.Add(Child);
			}
		}
	}
}

TSharedPtr<STrackLane> SOutlinerView::FindOrCreateParentLane(TViewModelPtr<IOutlinerExtension> InDataModel)
{
	TSharedPtr<FOutlinerViewModel> Outliner  = WeakOutliner.Pin();
	if (!Outliner)
	{
		return nullptr;
	}

	// Find any parent that will create a nested track lane
	for (const TViewModelPtr<ITrackAreaExtension>& ParentTrackArea : InDataModel.AsModel()->GetAncestorsOfType<ITrackAreaExtension>())
	{
		FTrackAreaParameters TrackAreaParameters = ParentTrackArea->GetTrackAreaParameters();
		if (TrackAreaParameters.LaneType == ETrackAreaLaneType::Nested)
		{
			// Find the first outliner extension parent (should be the same as ParentTrackArea really)
			if (auto AncestorsIt = ParentTrackArea.AsModel()->GetAncestorsOfType<IOutlinerExtension>(true); AncestorsIt)
			{
				const TViewModelPtr<IOutlinerExtension>& OutlinerExtItem = *AncestorsIt;
				TSharedPtr<STrackLane> TrackLane = TrackArea->FindTrackSlot(OutlinerExtItem);
				if (!TrackLane)
				{
					// Wire up the grandparent chain so PositionParentTrackLanes can climb past this lane
					// to ancestor lanes that own widgets like layer bars when this lane was never directly
					// created via CreateTrackLanesForRow.
					TSharedPtr<STrackLane> GrandparentLane = FindOrCreateParentLane(OutlinerExtItem);
					TrackLane = SNew(STrackLane, TrackArea, OutlinerExtItem, GrandparentLane, TrackAreaParameters, SharedThis(this));
					TrackArea->AddTrackSlot(OutlinerExtItem, TrackLane);
				}

				return TrackLane;
			}

			// Once we find any track area extension that wants nested lanes, return if it didn't manage to create one
			break;
		}
	}

	return nullptr;
}

void SOutlinerView::CreateTrackLanesForRow(TSharedRef<SOutlinerViewRow> InRow, TViewModelPtr<IOutlinerExtension> InDataModel)
{
	// Create a lane for this item, if needed, or find a lane to use from one of its parents.
	// We also create a lane on the appropriate parent if it doesn't exist, because we could be in a situation
	// where we jump-scrolled directly to the given row and item without ever having had the parent in view.
	const bool bIncludeDataModel = true;
	for (TViewModelPtr<IOutlinerExtension> CurDataModel : InDataModel.AsModel()->GetAncestorsOfType<IOutlinerExtension>(bIncludeDataModel))
	{
		if (TSharedPtr<ITrackAreaExtension> TrackAreaExtension = CurDataModel.ImplicitCast())
		{
			FTrackAreaParameters Parameters = TrackAreaExtension->GetTrackAreaParameters();
			if (Parameters.LaneType != ETrackAreaLaneType::None)
			{
				TSharedPtr<STrackLane> TrackLane = TrackArea->FindTrackSlot(CurDataModel);
				if (!TrackLane)
				{
					// Add a track slot for the row
					TSharedPtr<STrackLane> ParentLane = FindOrCreateParentLane(CurDataModel);
					TrackLane = SNew(STrackLane, TrackArea, CurDataModel, ParentLane, Parameters, SharedThis(this));

					TrackArea->AddTrackSlot(CurDataModel, TrackLane);
				}
				ensure(TrackLane);
				InRow->SetTrackLane(TrackLane);
				break;
			}
		}
	}
}

TSharedRef<ITableRow> SOutlinerView::OnGenerateRow(TWeakViewModelPtr<IOutlinerExtension> InWeakModel, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SOutlinerViewRow> Row =
		SNew(SOutlinerViewRow, OwnerTable, InWeakModel)
		.OnDetectDrag(this, &SOutlinerView::OnDragRow)
		.OnGetColumnVisibility(this, &SOutlinerView::IsColumnVisible)
		.OnGenerateWidgetForColumn(this, &SOutlinerView::GenerateWidgetForColumn);

	if (TViewModelPtr<IOutlinerExtension> ViewModel = InWeakModel.Pin())
	{
		CreateTrackLanesForRow(Row, ViewModel);
	}
	return Row;
}

TSharedRef<SWidget> SOutlinerView::GenerateWidgetForColumn(TViewModelPtr<IOutlinerExtension> InDataModel, const FName& ColumnId, const TSharedRef<SOutlinerViewRow>& Row) const
{
	TSharedPtr<FEditorViewModel> Editor = GetOutlinerModel()->GetEditor();
	if (!Editor)
	{
		return SNullWidget::NullWidget;
	}

	// First of all, see if the model wants to create a widget
	TSharedPtr<SWidget> ViewModelWidget = InDataModel->CreateOutlinerViewForColumn(FCreateOutlinerViewParams{ Row, Editor.ToSharedRef() }, ColumnId);
	if (ViewModelWidget)
	{
		return ViewModelWidget.ToSharedRef();
	}

	// Next see if we have a column generator function for this column
	if (const FColumnGenerator* Generator = ColumnGenerators.Find(ColumnId))
	{
		FCreateOutlinerColumnParams Params{ InDataModel, Editor };
		if (TSharedPtr<SWidget> Result = (*Generator)(Params, Row))
		{
			return Result.ToSharedRef();
		}
	}

	return SNullWidget::NullWidget;
}

bool SOutlinerView::IsColumnVisible(const FName& InName) const
{
	return Algo::FindBy(OutlinerColumns, InName, &IOutlinerColumn::GetColumnName) != nullptr;
}

} // namespace UE::Sequencer

