// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerViewRow.h"
#include "MVVM/Views/SOutlinerViewRowPanel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Views/STrackLane.h"
#include "MVVM/Extensions/ISelectableExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/EditorSharedViewModelData.h"
#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"


namespace UE::Sequencer
{

void SOutlinerViewRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TWeakViewModelPtr<IOutlinerExtension> InWeakModel)
{
	WeakModel = InWeakModel;

	OnDetectDrag = InArgs._OnDetectDrag;
	OnGetColumnVisibility = InArgs._OnGetColumnVisibility;
	OnGenerateWidgetForColumn = InArgs._OnGenerateWidgetForColumn;

	// Header row must be valid for this widget
	TSharedPtr<SHeaderRow> HeaderRow = OwnerTableView->GetHeaderRow();
	check(HeaderRow.IsValid());

	SeparatorHeight = InWeakModel.Pin()->GetOutlinerSizing().GetSeparatorHeight();

	STableRow::Construct(
		STableRow::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("Sequencer.Outliner.Row"))
		.OnDragDetected(this, &SOutlinerViewRow::OnDragDetected)
		.OnCanAcceptDrop(this, &SOutlinerViewRow::OnCanAcceptDrop)
		.OnAcceptDrop(this, &SOutlinerViewRow::OnAcceptDrop)
		.ShowSelection(IsSelectable())
		.Padding(FMargin(0.f))
		.Content()
		[
			SNew(SOutlinerViewRowPanel, HeaderRow.ToSharedRef())
			.OnGenerateCellContent(this, &SOutlinerViewRow::GenerateWidgetForColumn)
		],
		OwnerTableView);
}

SOutlinerViewRow::~SOutlinerViewRow()
{
}

void SOutlinerViewRow::ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent)
{
	this->Content = InContent;
	this->ChildSlot
	.Padding(InPadding)
	[
		InContent
	];
}

void SOutlinerViewRow::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	FVector2f Size = AllottedGeometry.GetLocalSize() - FVector2f(0.f, SeparatorHeight);
	ISequencerTreeViewRow::OnArrangeChildren(AllottedGeometry/*.MakeChild(Size, FSlateLayoutTransform())*/, ArrangedChildren);
}

int32 SOutlinerViewRow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = PaintBorder(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled) + 1;

	if (SeparatorHeight > 0.f)
	{
		FVector2f Size = AllottedGeometry.GetLocalSize();
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(
				FVector2f(Size.X, SeparatorHeight),
				FSlateLayoutTransform(FVector2f(0.f, Size.Y-SeparatorHeight))
			),
			FAppStyle::Get().GetBrush("WhiteTexture"),
			ESlateDrawEffect::None,
			FLinearColor(0.f, 0.f, 0.f, 1.f)
		);
	}

	LayerId = PaintSelection(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	LayerId = PaintDropIndicator(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return LayerId;
}

TSharedPtr<SWidget> SOutlinerViewRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	if (TViewModelPtr<IOutlinerExtension> DataModel = WeakModel.Pin())
	{
		return OnGenerateWidgetForColumn.Execute(DataModel, ColumnId, SharedThis(this));
	}
	return nullptr;
}

bool SOutlinerViewRow::IsColumnVisible(const FName& InColumnName) const
{
	return OnGetColumnVisibility.IsBound()
		? OnGetColumnVisibility.Execute(InColumnName)
		: true;
}

void SOutlinerViewRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	StaticCastSharedPtr<SOutlinerView>(OwnerTablePtr.Pin())->ReportChildRowGeometry(WeakModel.Pin(), AllottedGeometry);
}

bool SOutlinerViewRow::IsSelectable() const
{
	TSharedPtr<ISelectableExtension> Selectable = WeakModel.ImplicitPin();
	return !Selectable || Selectable->IsSelectable() != ESelectionIntent::Never;
}

FReply SOutlinerViewRow::OnDragDetected( const FGeometry& InGeometry, const FPointerEvent& InPointerEvent )
{
	if (OnDetectDrag.IsBound())
	{
		return OnDetectDrag.Execute(InGeometry, InPointerEvent, SharedThis(this));
	}
	return FReply::Unhandled();
}

int32 SOutlinerViewRow::OnPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Draw feedback for user dropping an item above, below, or onto a row.
	const FSlateBrush* DropIndicatorBrush = GetDropIndicatorBrush(InItemDropZone);

	// Offset by the indentation amount
	static float OffsetX = 10.0f;
	FVector2D Offset(OffsetX * GetIndentLevel(), 0.f);
	FSlateDrawElement::MakeBox
	(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(FVector2D(AllottedGeometry.GetLocalSize() - Offset), FSlateLayoutTransform(Offset)),
		DropIndicatorBrush,
		ESlateDrawEffect::None,
		DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	return LayerId;
}

TOptional<EItemDropZone> SOutlinerViewRow::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TWeakViewModelPtr<IOutlinerExtension> InDataModel)
{
	TViewModelPtr<IOutlinerExtension> ThisModel = InDataModel.Pin();
	if (!ThisModel)
	{
		return TOptional<EItemDropZone>();
	}
	else if (InItemDropZone == EItemDropZone::BelowItem && ThisModel->IsExpanded())
	{
		// Cannot drop immediately below items that are expanded since
		// it looks like you are dropping into the item, but the object
		// will end up below all its expanded children.
		return TOptional<EItemDropZone>();
	}

	// The model we are indirectly dropping into
	FViewModelPtr DropModel = ThisModel;
	// The model we are directly interacting with
	FViewModelPtr TargetModel;

	TSharedPtr<FOutlinerViewModelDragDropOp> OutlinerDragDropOp = DragDropEvent.GetOperationAs<FOutlinerViewModelDragDropOp>();
	if (OutlinerDragDropOp)
	{
		// Allow outliner drag operations to override drop model and target model
		OutlinerDragDropOp->GetTargetAndDropModel(ThisModel, InItemDropZone, DropModel, TargetModel);
	}
	else if (InItemDropZone == EItemDropZone::AboveItem || InItemDropZone == EItemDropZone::BelowItem)
	{
		TargetModel = DropModel;
		DropModel = DropModel->CastParent<IOutlinerDropTargetOutlinerExtension>();
	}

	// Check the pinned states of the dragged items vs the dropped items. Pinned items cannot be dragged onto unpinned items, and vice versa.
	if(OutlinerDragDropOp)
	{
		bool bSharedPinnedState = false;
		
		for (const TWeakViewModelPtr<IOutlinerExtension>& WeakDraggedModel : OutlinerDragDropOp->GetDraggedViewModels())
		{
			FViewModelPtr DraggedModel = WeakDraggedModel.Pin();
			if (!DraggedModel)
			{
				continue;
			}
			if (TSharedPtr<IPinnableExtension> DraggedPinnable = DraggedModel.ImplicitCast())
			{
				if (TSharedPtr<IPinnableExtension> PinnableTarget = TargetModel.ImplicitCast())
				{
					if (DraggedPinnable->IsPinned() == PinnableTarget->IsPinned())
					{
						bSharedPinnedState = true;
						break;
					}
				}
				if (TSharedPtr<IPinnableExtension> PinnableDropModel = DropModel.ImplicitCast())
				{
					if (DraggedPinnable->IsPinned() == PinnableDropModel->IsPinned())
					{
						bSharedPinnedState = true;
						break;
					}
				}
			}
			
		}
		
		// If no dragged items match the pinned state of the drop target, then they cannot be dropped.
		if (!bSharedPinnedState)
		{
			return TOptional<EItemDropZone>();
		}
	}

	TViewModelPtr<IOutlinerDropTargetOutlinerExtension> DropTarget = DropModel.ImplicitCast();
	if (!DropTarget)
	{
		return TOptional<EItemDropZone>();
	}

	return DropTarget->CanAcceptDrop(TargetModel, DragDropEvent, InItemDropZone);
}

FReply SOutlinerViewRow::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TWeakViewModelPtr<IOutlinerExtension> InDataModel)
{
	TViewModelPtr<IOutlinerExtension> ThisModel = InDataModel.Pin();
	if (!ThisModel)
	{
		return FReply::Unhandled();
	}
	else if (InItemDropZone == EItemDropZone::BelowItem && ThisModel->IsExpanded())
	{
		// Cannot drop immediately below items that are expanded since
		// it looks like you are dropping into the item, but the object
		// will end up below all its expanded children.
		return FReply::Unhandled();
	}

	// The model we are indirectly dropping into
	FViewModelPtr DropModel = ThisModel;
	// The model we are directly interacting with
	FViewModelPtr TargetModel;

	if (TSharedPtr<FOutlinerViewModelDragDropOp> OutlinerDragDropOp = DragDropEvent.GetOperationAs<FOutlinerViewModelDragDropOp>())
	{
		// Allow outliner drag operations to override drop model and target model
		OutlinerDragDropOp->GetTargetAndDropModel(ThisModel, InItemDropZone, DropModel, TargetModel);
	}
	else if (InItemDropZone == EItemDropZone::AboveItem || InItemDropZone == EItemDropZone::BelowItem)
	{
		TargetModel = DropModel;
		DropModel = DropModel->CastParent<IOutlinerDropTargetOutlinerExtension>();
	}

	TViewModelPtr<IOutlinerDropTargetOutlinerExtension> DropTarget = DropModel.ImplicitCast();
	if (DropTarget)
	{
		DropTarget->PerformDrop(TargetModel, DragDropEvent, InItemDropZone);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FVector2D SOutlinerViewRow::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D ReturnDesiredSize = SBorder::ComputeDesiredSize(LayoutScaleMultiplier);
	if (TrackLane && TrackLane->GetOutlinerItem())
	{
		// Ensure our height properly matches the height of the outliner item.
		ReturnDesiredSize.Y = TrackLane->GetOutlinerItem()->GetOutlinerSizing().GetTotalHeight();
	}
	return ReturnDesiredSize;
}

TViewModelPtr<IOutlinerExtension> SOutlinerViewRow::GetDataModel() const
{
	return WeakModel.Pin();
}

void SOutlinerViewRow::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TViewModelPtr<IOutlinerExtension>       DataModel        = WeakModel.Pin();
	TSharedPtr<FSharedViewModelData>        SharedData       = DataModel        ? DataModel.AsModel()->GetSharedData() : nullptr;
	TSharedPtr<FEditorSharedViewModelData>  SharedEditorData = SharedData       ? SharedData->CastThisShared<FEditorSharedViewModelData>() : nullptr;
	TSharedPtr<FEditorViewModel>            Editor           = SharedEditorData ? SharedEditorData->GetEditor() : nullptr;

	if (DataModel && Editor)
	{
		Editor->GetOutliner()->SetHoveredItem(DataModel);
	}
	SWidget::OnMouseEnter(MyGeometry, MouseEvent);
}

void SOutlinerViewRow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	TViewModelPtr<IOutlinerExtension>       DataModel        = WeakModel.Pin();
	TSharedPtr<FSharedViewModelData>        SharedData       = DataModel ? DataModel.AsModel()->GetSharedData() : nullptr;
	TSharedPtr<FEditorSharedViewModelData>  SharedEditorData = SharedData       ? SharedData->CastThisShared<FEditorSharedViewModelData>() : nullptr;
	TSharedPtr<FEditorViewModel>            Editor           = SharedEditorData ? SharedEditorData->GetEditor() : nullptr;

	if (Editor)
	{
		Editor->GetOutliner()->SetHoveredItem(nullptr);
	}

	SWidget::OnMouseLeave(MouseEvent);
}


TSharedPtr<STrackLane> SOutlinerViewRow::GetTrackLane(bool bOnlyOwnTrackLane) const
{
	if (!bOnlyOwnTrackLane)
	{
		// Return the track lane, regardless of it being created by our own outliner item, or
		// being a reference to a parent outliner item's track lane.
		return TrackLane;
	}
	if (TrackLane && TrackLane->GetOutlinerItem() == GetDataModel())
	{
		// Return the track lane only if it was created by our own outliner item.
		return TrackLane;
	}
	return nullptr;
}

void SOutlinerViewRow::SetTrackLane(const TSharedPtr<STrackLane>& InTrackLane)
{
	TrackLane = InTrackLane;
}

const FSlateBrush* SOutlinerViewRow::GetBorder() const 
{
	TSharedPtr<IOutlinerExtension> OutlinerItem = WeakModel.ImplicitPin();
	if (OutlinerItem && !OutlinerItem->HasBackground())
	{
		return nullptr;
	}

	return STableRow::GetBorder();
}

} // namespace UE::Sequencer

