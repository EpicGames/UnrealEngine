// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerViewRowPanel.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypesPrivate.h"
#include "Layout/LayoutUtils.h"

namespace UE::Sequencer
{

void SOutlinerViewRowPanel::Construct(const FArguments& InArgs, TSharedRef<SHeaderRow> InHeaderRow)
{
	WeakHeaderRow = InHeaderRow;
	GenerateCellContentEvent = InArgs._OnGenerateCellContent;

	InHeaderRow->OnColumnsChanged()->AddSP(this, &SOutlinerViewRowPanel::HandleColumnsUpdated);
	UpdateCells();
}

FVector2D SOutlinerViewRowPanel::ComputeDesiredSize(float) const
{
	return FVector2D(0.f, 0.f);
}

FChildren* SOutlinerViewRowPanel::GetChildren()
{
	return &Children;
}

void SOutlinerViewRowPanel::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	TSharedPtr<SHeaderRow> HeaderRow = WeakHeaderRow.Pin();
	if (!HeaderRow)
	{
		return;
	}

	const TIndirectArray<SHeaderRow::FColumn>&   Columns       = HeaderRow->GetColumns();
	TSharedPtr<FOutlinerHeaderRowWidgetMetaData> MetaDataEntry = HeaderRow->GetMetaData<FOutlinerHeaderRowWidgetMetaData>();

	check(MetaDataEntry && MetaDataEntry->Columns.Num() == Columns.Num());

	const int32 NumColumns = Columns.Num();
	const int32 NumSlots   = Children.Num();

	float RequiredFixedWidth      = 0.f;
	float TotalStretchCoefficient = 0.f;

	for (int32 ColumnIndex = 0; ColumnIndex < NumColumns; ++ColumnIndex)
	{
		// Compute the width of this column
		const SHeaderRow::FColumn&   Column = Columns[ColumnIndex];
		const FOutlinerColumnLayout& Layout = MetaDataEntry->Columns[ColumnIndex];

		if (Column.bIsVisible)
		{
			// Padding always contributes to the required width
			RequiredFixedWidth += Layout.CellPadding.GetTotalSpaceAlong<Orient_Horizontal>();

			if (Layout.SizeMode == EOutlinerColumnSizeMode::Stretch)
			{
				TotalStretchCoefficient += Layout.Width;
			}
			else
			{
				RequiredFixedWidth += Layout.Width;
			}
		}
	}

	if (RequiredFixedWidth <= 0.f && TotalStretchCoefficient <= 0.f)
	{
		return;
	}

	const float AllottedStretchSpace = FMath::Max(0.f, AllottedGeometry.GetLocalSize().X - RequiredFixedWidth);
	float       HorizontalOffset     = 0.f;

	auto ComputeCellWidth = [AllottedStretchSpace, TotalStretchCoefficient](const SHeaderRow::FColumn& Column, const FOutlinerColumnLayout& Layout)
	{
		float CellWidth = Layout.CellPadding.Left + Layout.CellPadding.Right;
		if (Layout.SizeMode == EOutlinerColumnSizeMode::Stretch)
		{
			CellWidth += TotalStretchCoefficient > 0.0f
				? AllottedStretchSpace * Layout.Width / TotalStretchCoefficient
				: 0.f;
		}
		else
		{
			CellWidth += Layout.Width;
		}
		return CellWidth;
	};

	// Now iterate all the columns again and arrange widgets
	for (int32 ColumnIndex = 0, SlotIndex = 0; ColumnIndex < NumColumns; ++ColumnIndex)
	{
		// Compute the width of this column
		const SHeaderRow::FColumn&   Column = Columns[ColumnIndex];
		const FOutlinerColumnLayout& Layout = MetaDataEntry->Columns[ColumnIndex];

		if (!Column.bIsVisible)
		{
			continue;
		}

		float CellWidth = ComputeCellWidth(Column, Layout);
		float PaddingEncroachment = 0.f;

		// If this column is for the next slot, handle any overflowing required and arrange the widget
		if (SlotIndex < NumSlots && Children[SlotIndex].ColumnId == Column.ColumnId)
		{
			const FSlot&        Slot   = Children[SlotIndex];
			TSharedRef<SWidget> Widget = Slot.GetWidget();

			EVisibility SlotVisibility = Widget->GetVisibility();

			const bool bOverflowEmpty = EnumHasAnyFlags(Layout.Flags, EOutlinerColumnFlags::OverflowSubsequentEmptyCells);
			const bool bForceOverflow = EnumHasAnyFlags(Layout.Flags, EOutlinerColumnFlags::OverflowOnHover) && Widget->IsHovered();

			if (bForceOverflow)
			{
				// When forcing overflow, we override every subsequent cell until we have enough space to meet the desired width
				const float RequiredWidth = Widget->GetDesiredSize().X + Layout.CellPadding.Left + Layout.CellPadding.Right;

				for (int32 NextColumnIndex=ColumnIndex+1;
					CellWidth < RequiredWidth && NextColumnIndex < NumColumns;
					++NextColumnIndex, ++ColumnIndex)
				{
					const SHeaderRow::FColumn&   NextColumn = Columns[NextColumnIndex];
					const FOutlinerColumnLayout& NextLayout = MetaDataEntry->Columns[NextColumnIndex];

					// Encroach the next cell's padding if possible without hiding the slot
					if (CellWidth + NextLayout.CellPadding.Left >= RequiredWidth)
					{
						CellWidth += NextLayout.CellPadding.Left;
						PaddingEncroachment = NextLayout.CellPadding.Left;
						break;
					}

					CellWidth += ComputeCellWidth(NextColumn, NextLayout);

					// If this belonged to the next slot, move past that as well
					if (SlotIndex+1 < NumSlots && NextColumn.ColumnId == Children[SlotIndex+1].ColumnId)
					{
						++SlotIndex;
					}
				}
			}

			if (bOverflowEmpty)
			{
				// Overflow any subsequent empty cells
				for (int32 NextColumnIndex=ColumnIndex+1;
					NextColumnIndex < NumColumns;
					++NextColumnIndex, ++ColumnIndex)
				{
					const SHeaderRow::FColumn&   NextColumn = Columns[NextColumnIndex];
					const FOutlinerColumnLayout& NextLayout = MetaDataEntry->Columns[NextColumnIndex];

					if (SlotIndex+1 < NumSlots && NextColumn.ColumnId == Children[SlotIndex+1].ColumnId)
					{
						const FSlot& NextSlot = Children[SlotIndex+1];
						if (NextSlot.GetWidget() == SNullWidget::NullWidget || NextSlot.GetWidget()->GetVisibility() == EVisibility::Collapsed)
						{
							CellWidth += ComputeCellWidth(NextColumn, NextLayout);
							++SlotIndex;
						}
						else
						{
							break;
						}
					}
					else
					{
						// If the next column doesn't have a slot, it is empty
						CellWidth += ComputeCellWidth(NextColumn, NextLayout);
					}
				}
			}

			if (SlotVisibility != EVisibility::Collapsed)
			{
				// Arrange the slot
				FVector2f SlotSize(CellWidth, AllottedGeometry.GetLocalSize().Y);

				struct
				{
					TSharedRef<SWidget> Widget;
					EHorizontalAlignment HAlign;
					EVerticalAlignment VAlign;

					TSharedRef<SWidget>  GetWidget() const { return Widget; }
					EHorizontalAlignment GetHorizontalAlignment() const { return HAlign; }
					EVerticalAlignment   GetVerticalAlignment() const { return VAlign; }

				} Child { Slot.GetWidget(), Layout.HAlign, Layout.VAlign };

				AlignmentArrangeResult XAlignmentResult = AlignChild<Orient_Horizontal>(SlotSize.X, Child, Layout.CellPadding);
				AlignmentArrangeResult YAlignmentResult = AlignChild<Orient_Vertical>(SlotSize.Y, Child, Layout.CellPadding);

				FVector2f Offset(HorizontalOffset + XAlignmentResult.Offset, YAlignmentResult.Offset);
				FVector2f Size(XAlignmentResult.Size, YAlignmentResult.Size);

				ArrangedChildren.AddWidget(SlotVisibility, AllottedGeometry.MakeChild(Slot.GetWidget(), Size, FSlateLayoutTransform(Offset)));
			}

			++SlotIndex;
		}

		HorizontalOffset += CellWidth - PaddingEncroachment;
	}
}

void SOutlinerViewRowPanel::HandleColumnsUpdated(const TSharedRef<SHeaderRow>&)
{
	UpdateCells();
}

void SOutlinerViewRowPanel::UpdateCells()
{
	TSharedPtr<SHeaderRow> HeaderRow = WeakHeaderRow.Pin();

	int32 ColumnIndex = 0;

	// If we have no header row any more, or no way to generate content, skip everything and destroy all the slots
	if (HeaderRow && GenerateCellContentEvent.IsBound())
	{
		const TIndirectArray<SHeaderRow::FColumn>&   Columns       = HeaderRow->GetColumns();
		TSharedPtr<FOutlinerHeaderRowWidgetMetaData> MetaDataEntry = HeaderRow->GetMetaData<FOutlinerHeaderRowWidgetMetaData>();
		if (!MetaDataEntry)
		{
			Children.Empty();
			return;
		}

		if (MetaDataEntry->Columns.Num() != Columns.Num())
		{
			// This should only happen if the columns have been cleared
			ensureMsgf(Columns.Num() == 0, TEXT("Column meta-data mismatch!"));
			// Just keep them in sync - this probably results in visual oddness but it's better than crashing or showing nothing
			MetaDataEntry->Columns.SetNum(Columns.Num());
		}

		// Look forwards in the array of existing children to find a matching column.
		//   When the columns have not changed this should find a match immediately and skip everything.
		//   Only if the columns have been reordered or new columns added/deleted will this do anything.
		const int32 NumHeaderColumns = Columns.Num();
		for (int32 HeaderColumnIndex = 0; HeaderColumnIndex < NumHeaderColumns; ++HeaderColumnIndex)
		{
			const SHeaderRow::FColumn& Column       = Columns[HeaderColumnIndex];
			const FName                ColumnId     = Column.ColumnId;
			const int32                NumChildren  = Children.Num();
			const bool                 bNeedsWidget = Column.bIsVisible && Column.ShouldGenerateWidget.Get(true);

			// Find an existing column widget for this header column starting from the current index
			for (int32 NextColumnIndex = ColumnIndex; NextColumnIndex < NumChildren; ++NextColumnIndex)
			{
				if (Children[NextColumnIndex].ColumnId == ColumnId)
				{
					if (!bNeedsWidget)
					{
						// This widget is no longer needed - move it to the end so it will get destroyed
						Children.Move(ColumnIndex, Children.Num()-1);
					}
					else if (NextColumnIndex != ColumnIndex)
					{
						// Move this column to the correct index if it isn't already
						Children.Move(NextColumnIndex, ColumnIndex);
					}
					break;
				}
			}

			if (!bNeedsWidget)
			{
				continue;
			}

			// If the current column is correct, move on
			const bool bReuseExisting = ColumnIndex < NumChildren && Children[ColumnIndex].ColumnId == ColumnId;
			if (bReuseExisting)
			{
				++ColumnIndex;
				continue;
			}

			// Columns can optionally be nullptr, which means we allocate no content at all for that cell,
			//    and other cells are able to use the space if their flags allow
			TSharedPtr<SWidget> CellContent = GenerateCellContentEvent.Execute(ColumnId);
			if (CellContent && CellContent != SNullWidget::NullWidget)
			{
				// Make a new slot for this column
				FSlot::FSlotArguments Args(MakeUnique<FSlot>());
				Args.ColumnId(ColumnId)
				[
					CellContent.ToSharedRef()
				];

				Children.InsertSlot(MoveTemp(Args), ColumnIndex);

				// Move on to the next column index
				++ColumnIndex;
			}
		}
	}

	// Remove any expired column widgets
	for (int32 ExpiredChild = Children.Num()-1; ExpiredChild >= ColumnIndex; --ExpiredChild)
	{
		Children.RemoveAt(ExpiredChild);
	}
}

} // namespace UE::Sequencer

