// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "SequencerCoreFwd.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"


namespace UE::Sequencer
{

class SOutlinerViewRowPanel
	: public SPanel
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGenerateCellContent, const FName&)

	SLATE_BEGIN_ARGS(SOutlinerViewRowPanel){}
		SLATE_EVENT(FOnGenerateCellContent, OnGenerateCellContent)
	SLATE_END_ARGS()

	SOutlinerViewRowPanel()
		: Children(this)
	{}

	void Construct(const FArguments& InArgs, TSharedRef<SHeaderRow> InHeaderRow);

private:

	void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;

	void HandleColumnsUpdated(const TSharedRef<SHeaderRow>&);

	void UpdateCells();

private:

	/*~ SPanel Interface */
	FVector2D ComputeDesiredSize(float) const override;
	FChildren* GetChildren() override;

private:

	struct FSlot : public TWidgetSlotWithAttributeSupport<FSlot>
	{
		SLATE_SLOT_BEGIN_ARGS(FSlot, TWidgetSlotWithAttributeSupport<FSlot>)
			SLATE_ARGUMENT(FName, ColumnId)
		SLATE_SLOT_END_ARGS()

		FName ColumnId;

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			ColumnId = InArgs._ColumnId;
			TWidgetSlotWithAttributeSupport<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
		}
	};

	TPanelChildren<FSlot> Children;
	TWeakPtr<SHeaderRow> WeakHeaderRow;
	FOnGenerateCellContent GenerateCellContentEvent;
};

} // namespace UE::Sequencer

