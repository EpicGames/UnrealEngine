// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/STrackAreaLaneView.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewSpace.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Extensions/IViewSpaceClientExtension.h"
#include "TimeToPixel.h"


namespace UE::Sequencer
{

void STrackAreaLaneView::Construct(const FArguments& InArgs, const FViewModelPtr& InViewModel, TSharedPtr<STrackAreaView> InTrackAreaView)
{
	WeakModel = InViewModel;
	WeakTrackAreaView = InTrackAreaView;

	// Use CastThisShared to ensure that we cast both native and dynamic extensions
	TSharedPtr<const IViewSpaceClientExtension> ViewSpaceClient = InViewModel->FindAncestorOfType<const IViewSpaceClientExtension>(true);

	FGuid ViewSpaceID = ViewSpaceClient ? ViewSpaceClient->GetViewSpaceID() : FGuid();
	TrackAreaTimeToPixel = InTrackAreaView->GetTimeToPixel(ViewSpaceID);

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

TSharedRef<const SWidget> STrackAreaLaneView::AsWidget() const
{
	return AsShared();
}

FTrackLaneScreenAlignment STrackAreaLaneView::GetAlignment(const ITrackLaneWidgetSpace& InScreenSpace, const FGeometry& InParentGeometry) const
{
	TSharedPtr<ITrackLaneExtension> TrackLaneExtension = WeakModel.ImplicitPin();
	if (TrackLaneExtension)
	{
		FTrackLaneVirtualAlignment VirtualAlignment = TrackLaneExtension->ArrangeVirtualTrackLaneView();
		return VirtualAlignment.ToScreen(InScreenSpace.GetScreenSpace(VirtualAlignment.ViewSpaceID), InParentGeometry);
	}
	return FTrackLaneScreenAlignment();
}

FTimeToPixel STrackAreaLaneView::GetRelativeTimeToPixel() const
{
	FTimeToPixel RelativeTimeToPixel = *TrackAreaTimeToPixel;

	TSharedPtr<ITrackLaneExtension> TrackLaneExtension = WeakModel.ImplicitPin();
	if (TrackLaneExtension)
	{
		FTrackLaneVirtualAlignment VirtualAlignment = TrackLaneExtension->ArrangeVirtualTrackLaneView();
		if (VirtualAlignment.Range.GetLowerBound().IsClosed())
		{
			RelativeTimeToPixel = TrackAreaTimeToPixel->RelativeTo(VirtualAlignment.Range.GetLowerBoundValue());
		}
	}

	return RelativeTimeToPixel;
}

} // namespace UE::Sequencer
