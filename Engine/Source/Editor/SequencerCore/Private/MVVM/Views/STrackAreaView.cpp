// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/STrackAreaView.h"

#include "Delegates/Delegate.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "ISequencerEditTool.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Geometry.h"
#include "Layout/LayoutUtils.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewSpace.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Views/STrackLane.h"
#include "Math/Range.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FrameNumber.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "TimeToPixel.h"
#include "Types/PaintArgs.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Widgets/SWeakWidget.h"
#include "Widgets/SWidget.h"

class FChildren;
class FDragDropEvent;
class FWidgetStyle;

namespace UE
{
namespace Sequencer
{

FTrackAreaSlot::FTrackAreaSlot(const TSharedPtr<STrackLane>& InSlotContent)
	: TAlignmentWidgetSlotMixin<FTrackAreaSlot>(HAlign_Fill, VAlign_Top)
{
	TrackLane = InSlotContent;

	this->AttachWidget(
		SNew(SWeakWidget)
			.PossiblyNullContent(InSlotContent)
	);
}

STrackAreaView::STrackAreaView()
	: Children(this)
{
}

STrackAreaView::~STrackAreaView()
{
}

void STrackAreaView::Construct(const FArguments& InArgs, TWeakPtr<FTrackAreaViewModel> InWeakViewModel)
{
	VirtualTop    = 0.f;

	WeakViewModel = InWeakViewModel;

	bShowPinnedNodes = false;

	// Input stack in order of priority

	// Space for the edit tool
	InputStack.AddHandler(nullptr);

	TSharedPtr<FTrackAreaViewModel> ViewModel = WeakViewModel.Pin();
	check(ViewModel);

	ViewModel->OnViewSpaceAdded.AddSP(this, &STrackAreaView::HandleViewSpaceAdded);
	ViewModel->OnViewSpaceRemoved.AddSP(this, &STrackAreaView::HandleViewSpaceRemoved);

	InputStack.OnBeginCapture.AddSP(ViewModel.Get(), &FTrackAreaViewModel::LockEditTool);
	InputStack.OnEndCapture.AddSP(ViewModel.Get(), &FTrackAreaViewModel::UnlockEditTool);

	// Set some default time to pixel
	TimeToPixel = MakeShared<FTimeToPixel>(ViewModel->GetTimeToPixel(100.f));

	SetClipping(EWidgetClipping::ClipToBoundsAlways);
}

const FTrackAreaViewLayers& STrackAreaView::GetPaintLayers() const
{
	return LayerIds;
}

TSharedPtr<FTrackAreaViewModel> STrackAreaView::GetViewModel() const
{
	return WeakViewModel.Pin();
}

TSharedPtr<FTimeToPixel> STrackAreaView::GetFallbackTimeToPixel() const
{
	return TimeToPixel;
}

TSharedPtr<FTimeToPixel> STrackAreaView::GetTimeToPixel(const FGuid& InViewSpaceID) const
{
	TSharedPtr<FTimeToPixel> Result = ViewSpaceTimeToPixel.FindRef(InViewSpaceID);
	return Result ? Result : TimeToPixel;
}

FTimeToPixel STrackAreaView::GetScreenSpace(const FGuid& InViewSpaceID) const
{
	return *GetTimeToPixel(InViewSpaceID);
}

void STrackAreaView::HandleViewSpaceAdded(const FGuid& InID, TSharedPtr<FTrackAreaViewSpace> ViewSpace)
{
	TSharedPtr<FTimeToPixel> TTP = MakeShared<FTimeToPixel>(TimeToPixel->CreateNonLinear(ViewSpace));
	ensure(TTP->NonLinearTransform);
	ViewSpaceTimeToPixel.Add(InID, TTP);
}

void STrackAreaView::HandleViewSpaceRemoved(const FGuid& InID, TSharedPtr<FTrackAreaViewSpace> ViewSpace)
{
	ViewSpaceTimeToPixel.Remove(InID);
}

FLinearColor STrackAreaView::BlendDefaultTrackColor(FLinearColor InColor)
{
	static FLinearColor BaseColor(FColor(71,71,71));

	const float Alpha = InColor.A;
	InColor.A = 1.f;
	
	return BaseColor * (1.f - Alpha) + InColor * Alpha;
}

void STrackAreaView::SetOutliner(const TSharedPtr<SOutlinerView>& InOutliner)
{
	WeakOutliner = InOutliner;
}

void STrackAreaView::Empty()
{
	TrackSlots.Empty();
	Children.Empty();
}

void STrackAreaView::PurgeDeadTrackSlots()
{
	for (auto It = TrackSlots.CreateIterator(); It; ++It)
	{
		if (!It.Value().IsValid())
		{
			It.RemoveCurrent();
		}
	}

	for (int32 ChildIndex = Children.Num() - 1; ChildIndex >= 0; --ChildIndex)
	{
		if (!Children[ChildIndex].TrackLane.IsValid())
		{
			Children.RemoveAt(ChildIndex);
		}
	}
}

void STrackAreaView::AddTrackSlot(const TViewModelPtr<IOutlinerExtension>& InModel, const TSharedPtr<STrackLane>& InSlot)
{
	TrackSlots.Add(InModel, InSlot);
	Children.AddSlot(FTrackAreaSlot::FSlotArguments(MakeUnique<FTrackAreaSlot>(InSlot)));
}


TSharedPtr<STrackLane> STrackAreaView::FindTrackSlot(const TViewModelPtr<IOutlinerExtension>& InModel)
{
	return TrackSlots.FindRef(InModel).Pin();
}


void STrackAreaView::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	if (!TrackArea)
	{
		return;
	}

	*const_cast<STrackAreaView*>(this)->TimeToPixel = FTimeToPixel(TrackArea->GetTimeToPixel(AllottedGeometry.GetLocalSize().X));

	for (TPair<FGuid, TSharedPtr<FTimeToPixel>>& Pair : ViewSpaceTimeToPixel)
	{
		*Pair.Value = TimeToPixel->CreateNonLinear(Pair.Value->NonLinearTransform);
	}

	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FTrackAreaSlot& CurChild = Children[ChildIndex];

		const EVisibility ChildVisibility = CurChild.GetWidget()->GetVisibility();
		if (!ArrangedChildren.Accepts(ChildVisibility))
		{
			continue;
		}

		TSharedPtr<STrackLane> PinnedTrackLane = CurChild.TrackLane.Pin();
		if (!PinnedTrackLane.IsValid() || PinnedTrackLane->IsPinned() != bShowPinnedNodes)
		{
			continue;
		}


		const float PhysicalPosition = PinnedTrackLane->GetVerticalPosition();
		const FMargin Padding(0, PhysicalPosition, 0, 0);

		AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(AllottedGeometry.GetLocalSize().X, CurChild, Padding, 1.0f, false);
		AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, CurChild, Padding, 1.0f, false);

		ArrangedChildren.AddWidget(ChildVisibility,
			AllottedGeometry.MakeChild(
				CurChild.GetWidget(),
				FVector2D(XResult.Offset,YResult.Offset),
				FVector2D(XResult.Size, YResult.Size)
			)
		);
	}
}


FVector2D STrackAreaView::ComputeDesiredSize( float ) const
{
	FVector2D MaxSize(0,0);
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FTrackAreaSlot& CurChild = Children[ChildIndex];

		const EVisibility ChildVisibilty = CurChild.GetWidget()->GetVisibility();
		if (ChildVisibilty != EVisibility::Collapsed)
		{
			FVector2D ChildDesiredSize = CurChild.GetWidget()->GetDesiredSize();
			MaxSize.X = FMath::Max(MaxSize.X, ChildDesiredSize.X);
			MaxSize.Y = FMath::Max(MaxSize.Y, ChildDesiredSize.Y);
		}
	}

	return MaxSize;
}


FChildren* STrackAreaView::GetChildren()
{
	return &Children;
}


int32 STrackAreaView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	if (!TrackArea)
	{
		return LayerId;
	}

	// Reassign the time <-> pixel space for this frame
	*TimeToPixel = TrackArea->GetTimeToPixel(AllottedGeometry.GetLocalSize().X);
	for (TPair<FGuid, TSharedPtr<FTimeToPixel>>& Pair : ViewSpaceTimeToPixel)
	{
		*Pair.Value = TimeToPixel->CreateNonLinear(Pair.Value->NonLinearTransform);
	}

	// TODO: TrackEditors are drawn in subclass, maybe should be done in viewmodel?

	// paint the child widgets
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	const FPaintArgs NewArgs = Args.WithNewParent(this);

	LayerIds.LaneBackgrounds = LayerId;

	float MaxLayerId = LayerId;
	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		FSlateRect ChildClipRect = MyCullingRect.IntersectionWith( CurWidget.Geometry.GetLayoutBoundingRect() );
		const int32 ThisWidgetLayerId = CurWidget.Widget->Paint( NewArgs, CurWidget.Geometry, ChildClipRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) );

		MaxLayerId = FMath::Max(MaxLayerId, ThisWidgetLayerId);
	}

	LayerId = MaxLayerId;

	if (const ISequencerEditTool* EditTool = TrackArea->GetEditTool())
	{
		LayerId = EditTool->OnPaint(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + 1);
	}

	// TODO: where is highlight region?

	TViewModelPtr<IOutlinerExtension> DroppedItem = WeakDroppedItem.Pin();

	// Draw drop target
	if (DroppedItem && TrackSlots.Contains(DroppedItem))
	{
		TSharedPtr<STrackLane> TrackLane = TrackSlots.FindRef(DroppedItem).Pin();

		FSlateColor DashColor = bAllowDrop ? FStyleColors::AccentBlue : FStyleColors::Error;

		const FSlateBrush* HorizontalBrush = FAppStyle::GetBrush("WideDash.Horizontal");
		const FSlateBrush* VerticalBrush = FAppStyle::GetBrush("WideDash.Vertical");

		const float TrackPosition = TrackLane->GetVerticalPosition();
		const float TrackHeight   = TrackLane->GetDesiredSize().Y;

		int32 DashLayer = LayerId + 1;

		float DropMinX = 0.f;
		float DropMaxX = TrackHeight;

		if (DropFrameRange.IsSet())
		{
			DropMinX = TimeToPixel->FrameToPixel(DropFrameRange.GetValue().GetLowerBoundValue());
			DropMaxX = TimeToPixel->FrameToPixel(DropFrameRange.GetValue().GetUpperBoundValue());
		}

		// Top
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			DashLayer,
			AllottedGeometry.ToPaintGeometry(FVector2f(DropMaxX-DropMinX, HorizontalBrush->ImageSize.Y), FSlateLayoutTransform(FVector2f(DropMinX, TrackPosition))),
			HorizontalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());

		// Bottom
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			DashLayer,
			AllottedGeometry.ToPaintGeometry(FVector2f(DropMaxX-DropMinX, HorizontalBrush->ImageSize.Y), FSlateLayoutTransform(FVector2f(DropMinX, TrackPosition + (TrackHeight - HorizontalBrush->ImageSize.Y)))),
			HorizontalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());

		// Left
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			DashLayer,
			AllottedGeometry.ToPaintGeometry(FVector2f(VerticalBrush->ImageSize.X, TrackHeight), FSlateLayoutTransform(FVector2f(DropMinX, TrackPosition))),
			VerticalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());

		// Right
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			DashLayer,
			AllottedGeometry.ToPaintGeometry(FVector2f(VerticalBrush->ImageSize.X, TrackHeight), FSlateLayoutTransform(FVector2f(DropMaxX - VerticalBrush->ImageSize.X, TrackPosition))),
			VerticalBrush,
			ESlateDrawEffect::None,
			DashColor.GetSpecifiedColor());
	}

	return LayerId;
}

FReply STrackAreaView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool* EditTool = TrackArea ? TrackArea->GetEditTool() : nullptr;

	if (EditTool)
	{
		// Always ensure the edit tool is set up
		InputStack.SetHandlerAt(0, EditTool);
	}
	return InputStack.HandleKeyDown(*this, MyGeometry, InKeyEvent);
}

FReply STrackAreaView::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool* EditTool = TrackArea ? TrackArea->GetEditTool() : nullptr;

	if (EditTool)
	{
		// Always ensure the edit tool is set up
		InputStack.SetHandlerAt(0, EditTool);
	}
	return InputStack.HandleKeyUp(*this, MyGeometry, InKeyEvent);
}

FReply STrackAreaView::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool*             EditTool  = TrackArea ? TrackArea->GetEditTool() : nullptr;

	if (EditTool)
	{
		// Always ensure the edit tool is set up
		InputStack.SetHandlerAt(0, EditTool);
	}
	return InputStack.HandleMouseButtonDown(*this, MyGeometry, MouseEvent);
}


FReply STrackAreaView::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool*             EditTool  = TrackArea ? TrackArea->GetEditTool() : nullptr;

	if (EditTool)
	{
		// Always ensure the edit tool is set up
		InputStack.SetHandlerAt(0, EditTool);
	}
	return InputStack.HandleMouseButtonUp(*this, MyGeometry, MouseEvent);
}

FReply STrackAreaView::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool*             EditTool  = TrackArea ? TrackArea->GetEditTool() : nullptr;

	if (EditTool)
	{
		// Always ensure the edit tool is set up
		InputStack.SetHandlerAt(0, EditTool);
	}
	return InputStack.HandleMouseDoubleClick(*this, MyGeometry, MouseEvent);
}

void STrackAreaView::UpdateHoverStates( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<SOutlinerView>       PinnedOutliner = WeakOutliner.Pin();
	TSharedPtr<FTrackAreaViewModel> ViewModel      = WeakViewModel.Pin();

	// Set the node that we are hovering
	TViewModelPtr<IOutlinerExtension> NewHoveredItem = PinnedOutliner->HitTestNode(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).Y);

	TSharedPtr<FOutlinerViewModel> OutlinerModel = PinnedOutliner->GetOutlinerModel();

	bool bHoveredItemChanging = OutlinerModel && (NewHoveredItem != OutlinerModel->GetHoveredItem());
	// Suppress hover for items outside the drag owner's track during section drag
	if (bHoveredItemChanging && NewHoveredItem && ViewModel)
	{
		if (ISequencerEditTool* EditTool = ViewModel->GetEditTool())
		{
			if (ISequencerEditToolDragOperation* DragOp = EditTool->GetCurrentDragOperation())
			{
				if (TSharedPtr<FViewModel> DragOwner = DragOp->GetDragOwnerTrackModel())
				{
					if (!DragOwner->IsAncestorOf(NewHoveredItem.AsModel(), /*bIncludeThis*/ true))
					{
						NewHoveredItem = nullptr;
					}
				}
			}
		}
	}

	if (OutlinerModel)
	{
		OutlinerModel->SetHoveredItem(NewHoveredItem);
	}

	TSharedPtr<ITrackAreaHotspot> Hotspot = ViewModel->GetHotspot();
	if (Hotspot.IsValid())
	{
		Hotspot->UpdateOnHover(*ViewModel);
	}
}

FReply STrackAreaView::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool*             EditTool  = TrackArea ? TrackArea->GetEditTool() : nullptr;

	UpdateHoverStates(MyGeometry, MouseEvent);

	if (EditTool)
	{
		// Always ensure the edit tool is set up
		InputStack.SetHandlerAt(0, EditTool);
	}

	FReply Reply = InputStack.HandleMouseMove(*this, MyGeometry, MouseEvent);

	// Handle right click scrolling on the track area, if the captured index is that of the time slider
	if (Reply.IsEventHandled() && InputStack.GetCapturedIndex() == 1)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && HasMouseCapture())
		{
			if (const TSharedPtr<SOutlinerView> Outliner = WeakOutliner.Pin())
			{
				Outliner->ScrollByDelta(-MouseEvent.GetCursorDelta().Y);
			}
		}
	}

	return Reply;
}


FReply STrackAreaView::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool*             EditTool  = TrackArea ? TrackArea->GetEditTool() : nullptr;

	if (EditTool)
	{
		// Always ensure the edit tool is set up
		InputStack.SetHandlerAt(0, EditTool);
	}

	FReply EditToolHandle = InputStack.HandleMouseWheel(*this, MyGeometry, MouseEvent);
	if (EditToolHandle.IsEventHandled())
	{
		return EditToolHandle;
	}

	if (const TSharedPtr<SOutlinerView> Outliner = WeakOutliner.Pin())
	{
		const float ScrollAmount = -MouseEvent.GetWheelDelta() * GetGlobalScrollAmount();
		Outliner->ScrollByDelta(ScrollAmount);
	}

	return FReply::Handled();
}


void STrackAreaView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	WeakDroppedItem = nullptr;
	bAllowDrop = false;
	DropFrameRange.Reset();

	TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin();
	ISequencerEditTool* const EditTool = TrackArea ? TrackArea->GetEditTool() : nullptr;

	if (ISequencerInputHandler* const InputHandler = EditTool)
	{
		InputHandler->OnMouseEnter(*this, MyGeometry, MouseEvent);
	}
}


void STrackAreaView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<SOutlinerView> Outliner = WeakOutliner.Pin())
	{
		TSharedPtr<FOutlinerViewModel> OutlinerModel = Outliner->GetOutlinerModel();
		if (OutlinerModel)
		{
			OutlinerModel->SetHoveredItem(nullptr);
		}
	}

	if (TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin())
	{
		if (ISequencerInputHandler* const ToolInputHandler = TrackArea->GetEditTool())
		{
			ToolInputHandler->OnMouseLeave(*this, MouseEvent);
		}
	}
}


void STrackAreaView::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	if (TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin())
	{
		if (ISequencerInputHandler* const ToolInputHandler = TrackArea->GetEditTool())
		{
			ToolInputHandler->OnMouseCaptureLost();
		}
	}
}


FCursorReply STrackAreaView::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if (CursorEvent.IsMouseButtonDown(EKeys::RightMouseButton) && HasMouseCapture())
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
	}

	if (TSharedPtr<FTrackAreaViewModel> TrackArea = WeakViewModel.Pin())
	{
		if (TSharedPtr<ITrackAreaHotspot> Hotspot = TrackArea->GetHotspot())
		{
			FCursorReply HotspotCursor = Hotspot->GetCursor();
			if (HotspotCursor.IsEventHandled())
			{
				return HotspotCursor;
			}
		}

		if (ISequencerInputHandler* const ToolInputHandler = TrackArea->GetEditTool())
		{
			return ToolInputHandler->OnCursorQuery(*this, MyGeometry, CursorEvent);
		}
	}

	return FCursorReply::Unhandled();
}


void STrackAreaView::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	FVector2D Size = AllottedGeometry.GetLocalSize();

	if (!IsPinned() && SizeLastFrame.IsSet() && Size.X != SizeLastFrame->X)
	{
		OnResized(SizeLastFrame.GetValue(), Size);
	}

	SizeLastFrame = AllottedGeometry.GetLocalSize();
	
	// TODO: where did "Zoom by the difference" go?

	for (int32 Index = 0; Index < Children.Num();)
	{
		if (!StaticCastSharedRef<SWeakWidget>(Children[Index].GetWidget())->ChildWidgetIsValid())
		{
			Children.RemoveAt(Index);
		}
		else
		{
			++Index;
		}
	}
}

void STrackAreaView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SPanel::OnDragEnter(MyGeometry, DragDropEvent);
}

void STrackAreaView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	WeakDroppedItem = nullptr;
	SPanel::OnDragLeave(DragDropEvent);
}

FReply STrackAreaView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	return SPanel::OnDragOver(MyGeometry, DragDropEvent);
}

FReply STrackAreaView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	return SPanel::OnDrop(MyGeometry, DragDropEvent);
}

} // namespace Sequencer
} // namespace UE

