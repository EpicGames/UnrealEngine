// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/STrackLane.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Geometry.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "MVVM/Extensions/IHoveredExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/IResizableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Views/STrackAreaView.h"
#include "Math/Color.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SequencerCoreFwd.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "TimeToPixel.h"
#include "Types/PaintArgs.h"
#include "UObject/NameTypes.h"
#include "Widgets/SWidget.h"

class FWidgetStyle;
namespace UE::Sequencer { class FOutlinerViewModel; }

namespace UE
{
namespace Sequencer
{

STrackLane::FSlot::FSlot(TSharedPtr<ITrackLaneWidget> InInterface)
	: Interface(InInterface.ToSharedRef())
{
}

STrackLane::FSlot::~FSlot()
{
}

STrackLane::STrackLane()
	: Children(this)
{
}

STrackLane::~STrackLane()
{
}

void STrackLane::Construct(const FArguments& InArgs, TWeakPtr<STrackAreaView> InTrackAreaView, TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerItem, TSharedPtr<STrackLane> InParentLane, const FTrackAreaParameters& InTrackParams, const TSharedRef<SOutlinerView>& InTreeView)
{
	bWidgetsDirty = true;

	WeakTrackAreaView = InTrackAreaView;
	WeakOutlinerItem = InWeakOutlinerItem;
	TreeView = InTreeView;
	ParentLane = InParentLane;

	if (ParentLane)
	{
		ParentLane->WeakChildLanes.Add(SharedThis(this));
	}

	TrackParams = InTrackParams;
	TimeToPixel = InTrackAreaView.Pin()->GetTimeToPixel();

	FViewModelPtr OutlinerItem = GetOutlinerItem();
	if (ensure(OutlinerItem))
	{
		TSharedPtr<FSharedViewModelData> SharedData = OutlinerItem->GetSharedData();
		SharedData->SubscribeToHierarchyChanged(OutlinerItem)
			.AddSP(this, &STrackLane::OnHierarchyUpdated);
	}

	RecreateWidgets();

	SetVisibility(EVisibility::SelfHitTestInvisible);
}

TViewModelPtr<IOutlinerExtension> STrackLane::GetOutlinerItem() const
{
	return WeakOutlinerItem.Pin();
}

TSharedPtr<STrackLane> STrackLane::GetParentLane() const
{
	return ParentLane;
}

TSharedPtr<STrackAreaView> STrackLane::GetTrackAreaView() const
{
	return WeakTrackAreaView.Pin();
}

TSharedPtr<FTimeToPixel> STrackLane::GetTimeToPixel() const
{
	return TimeToPixel;
}

TArrayView<const TWeakPtr<STrackLane>> STrackLane::GetChildLanes() const
{
	return WeakChildLanes;
}

bool STrackLane::IsPinned() const
{
	TSharedPtr<IOutlinerExtension> Outliner = WeakOutlinerItem.Pin();
	TSharedPtr<IPinnableExtension> Pinnable = WeakOutlinerItem.ImplicitPin();

	if ((Pinnable && Pinnable->IsPinned()) || (Outliner && Outliner->ShouldAnchorToTop()))
	{
		return true;
	}

	// When the parent lane is pinned, then the child lane should also act as if it was pinned.
	return ParentLane && ParentLane->IsPinned();
}

void STrackLane::OnHierarchyUpdated()
{
	bWidgetsDirty = true;
}

TSharedPtr<ITrackLaneWidget> STrackLane::FindWidgetForModel(const FWeakViewModelPtr& InModel) const
{
	return WidgetsByModel.FindRef(InModel);
}

void STrackLane::RecreateWidgets()
{
	if (!bWidgetsDirty)
	{
		return;
	}

	bWidgetsDirty = false;
	Children.Empty();
	WidgetsByModel.Reset();

	WeakChildLanes.Remove(TWeakPtr<STrackLane>());

	TSharedPtr<STrackAreaView>      TrackAreaView      = WeakTrackAreaView.Pin();
	TSharedPtr<ITrackAreaExtension> TrackAreaExtension = WeakOutlinerItem.ImplicitPin();
	TSharedPtr<SOutlinerView>       PinnedTree         = TreeView.Pin();
	if (!TrackAreaView || !TrackAreaExtension || !PinnedTree)
	{
		return;
	}

	FCreateTrackLaneViewParams ViewParams(TrackAreaView->GetViewModel()->GetEditor());
	ViewParams.OwningTrackLane = SharedThis(this);
	ViewParams.TimeToPixel = TimeToPixel;

	auto ConstructTrackLaneView = [&](TViewModelPtr<ITrackLaneExtension> Model)
	{
		TSharedPtr<ITrackLaneWidget> ParentView;

		if (ParentLane)
		{
			FViewModelPtr Parent = Model.AsModel()->GetParent();
			// Find the parent model that exists in the parent lane
			while (Parent && !ParentView)
			{
				ParentView = ParentLane->FindWidgetForModel(Parent);
				Parent = Parent->GetParent();
			}
		}

		TSharedPtr<ITrackLaneWidget> NewView = Model->CreateTrackLaneView(ViewParams);
		WidgetsByModel.Add(Model, NewView);

		// If we have a parent view, add the widget to that
		if (ParentView)
		{
			ParentView->AddChildView(NewView, SharedThis(this));
		}
		// Otherwise we add it directly to this lane
		else
		{
			FSlot::FSlotArguments SlotArguments(MakeUnique<FSlot>(NewView));
			SlotArguments.AttachWidget(NewView->AsWidget());
			Children.AddSlot(MoveTemp(SlotArguments));
		}
	};

	// Construct views for this track lane
	for (TTypedIterator<ITrackLaneExtension, FViewModelVariantIterator> It(TrackAreaExtension->GetTrackAreaModelList()); It; ++It)
	{
		ConstructTrackLaneView(*It);
	}

	for (TTypedIterator<ITrackLaneExtension, FViewModelVariantIterator> It(TrackAreaExtension->GetTopLevelChildTrackAreaModels()); It; ++It)
	{
		ConstructTrackLaneView(*It);
	}
}

void STrackLane::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	TSharedPtr<STrackAreaView> TrackAreaView = WeakTrackAreaView.Pin();

	for (int32 WidgetIndex = 0; WidgetIndex < Children.Num(); ++WidgetIndex)
	{
		const FSlot&        Slot   = Children[WidgetIndex];
		TSharedRef<SWidget> Widget = Slot.GetWidget();

		EVisibility WidgetVisibility = Widget->GetVisibility();
		if (ArrangedChildren.Accepts(WidgetVisibility))
		{
			FTrackLaneScreenAlignment ScreenAlignment = Slot.Interface->GetAlignment(*TrackAreaView, AllottedGeometry);
			if (ScreenAlignment.IsVisible())
			{
				FArrangedWidget ArrangedWidget = ScreenAlignment.ArrangeWidget(Widget, AllottedGeometry);
				ArrangedChildren.AddWidget(WidgetVisibility, ArrangedWidget);
			}
		}
	}
}

int32 STrackLane::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		FSlateRect ChildClipRect = MyCullingRect.IntersectionWith( CurWidget.Geometry.GetLayoutBoundingRect() );
		LayerId = CurWidget.Widget->Paint( Args.WithNewParent(this), CurWidget.Geometry, ChildClipRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) );
	}

	LayerId = PaintLaneForeground(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle);

	return LayerId + 1;
}

int32 STrackLane::PaintLaneForeground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) const
{
	static const FName BorderName("Sequencer.AnimationOutliner.DefaultBorder");
	static const FName SelectionColorName("SelectionColor");

	TSharedPtr<STrackAreaView>        TrackAreaView = WeakTrackAreaView.Pin();;
	TViewModelPtr<IOutlinerExtension> OutlinerItem  = WeakOutlinerItem.Pin();
	if (!TrackAreaView || !OutlinerItem || OutlinerItem->IsFilteredOut())
	{
		return LayerId;
	}

	TViewModelPtr<IHoveredExtension> Hoverable = OutlinerItem.ImplicitCast();

	float TotalNodeHeight = OutlinerItem->GetOutlinerSizing().GetTotalHeight();

	const EOutlinerSelectionState SelectionState = OutlinerItem->GetSelectionState();

	// draw selection border
	if (SelectionState == EOutlinerSelectionState::SelectedDirectly)
	{
		FLinearColor SelectionColor = FAppStyle::GetSlateColor(SelectionColorName).GetColor(InWidgetStyle);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(
				FVector2f(AllottedGeometry.GetLocalSize().X, TotalNodeHeight),
				FSlateLayoutTransform()
			),
			FAppStyle::GetBrush(BorderName),
			ESlateDrawEffect::None,
			SelectionColor.CopyWithNewOpacity(0.2f)
		);
	}

	// draw hovered or node has keys or sections selected border
	else
	{
		FLinearColor HighlightColor;
		bool bDrawHighlight = false;
		if (SelectionState != EOutlinerSelectionState::None)
		{
			bDrawHighlight = true;
			HighlightColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.15f);
		}
		else if (Hoverable && Hoverable->IsHovered())
		{
			bDrawHighlight = true;
			HighlightColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.05f);
		}

		if (bDrawHighlight)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(
					FVector2f(AllottedGeometry.GetLocalSize().X, TotalNodeHeight),
					FSlateLayoutTransform()
				),
				FAppStyle::GetBrush(BorderName),
				ESlateDrawEffect::None,
				HighlightColor
			);
		}
	}

	return LayerId;
}

void STrackLane::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bWidgetsDirty)
	{
		RecreateWidgets();
	}
	
	// Sort children so they can be drawn bottom to top
	Children.StableSort([](const FSlot& A, const FSlot& B)
		{
			return A.Interface->GetOverlapPriority() < B.Interface->GetOverlapPriority();
		});

	FVector2D ThisFrameDesiredSize = GetDesiredSize();

	if (LastDesiredSize.IsSet() && ThisFrameDesiredSize.Y != LastDesiredSize.GetValue().Y)
	{
		TSharedPtr<SOutlinerView> PinnedTree = TreeView.Pin();
		if (PinnedTree.IsValid())
		{
			PinnedTree->RequestTreeRefresh();
		}
	}

	LastDesiredSize = ThisFrameDesiredSize;

	for (int32 WidgetIndex = 0; WidgetIndex < Children.Num(); ++WidgetIndex)
	{
		Children[WidgetIndex].Interface->ReportParentGeometry(AllottedGeometry);
	}
}

FVector2D STrackLane::ComputeDesiredSize(float LayoutScale) const
{
	TViewModelPtr<IOutlinerExtension>  OutlinerItem = WeakOutlinerItem.Pin();
	TViewModelPtr<ITrackAreaExtension> TrackArea    = OutlinerItem.ImplicitCast();

	float Height = 0.f;

	if (TrackArea && OutlinerItem)
	{
		FTrackAreaParameters Parameters = TrackArea->GetTrackAreaParameters();

		Height = OutlinerItem->GetOutlinerSizing().GetTotalHeight();

		// Include child heights if necessary
		if (Parameters.LaneType == ETrackAreaLaneType::Nested && OutlinerItem->IsExpanded())
		{
			for (auto It = OutlinerItem.AsModel()->GetDescendantsOfType<IOutlinerExtension>(); It; ++It)
			{
				if ((*It)->IsFilteredOut())
				{
					It.IgnoreCurrentChildren();
					continue;
				}

				Height += It->GetOutlinerSizing().GetTotalHeight();

				if (!(*It)->IsExpanded())
				{
					It.IgnoreCurrentChildren();
				}
			}
		}

		Height -= Parameters.TrackLanePadding.Top + Parameters.TrackLanePadding.Bottom;
	}

	return FVector2D(100.f, Height);
}

void STrackLane::SetVerticalPosition(float InPosition)
{
	Position = InPosition;
}

float STrackLane::GetVerticalPosition() const
{
	if (TViewModelPtr<ITrackAreaExtension> TrackArea = WeakOutlinerItem.ImplicitPin())
	{
		return Position + TrackArea->GetTrackAreaParameters().TrackLanePadding.Top;
	}
	return Position;
}

void STrackLane::PositionParentTrackLanes(TViewModelPtr<IOutlinerExtension> InItem, float InItemTop)
{
	// If the given item is our own item, our position is the same as its position. Simply set it,
	// and start positioning our parent.
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerItem.Pin();
	if (InItem == OutlinerItem)
	{
		SetVerticalPosition(InItemTop);

		if (ParentLane)
		{
			ParentLane->PositionParentTrackLanes(OutlinerItem, InItemTop);
		}

		return;
	}

	// The given item is somewhere inside the descendants of our owning outliner item. Walk up its hierarchy
	// and accumulate the height of all siblings that come before (above) it. We might have to go up by
	// multiple levels until we hit our owning outliner item. Also, to be safe, we need to support having
	// non-outliner items mixed in with the rest.
	float AccumulatedItemTop = InItemTop;
	TSharedPtr<FViewModel> StopAtChild = InItem.AsModel();
	TSharedPtr<FViewModel> ParentItem = StopAtChild->GetParent();
	while (ParentItem)
	{
		constexpr bool bIncludeThis = true;
		for (FParentFirstChildIterator It(ParentItem, bIncludeThis); It; ++It)
		{
			if (*It == StopAtChild)
			{
				break;
			}

			if (IOutlinerExtension* OutlinerExtension = It->CastThis<IOutlinerExtension>())
			{
				const bool bIsExpanded = OutlinerExtension->IsExpanded();
				const bool bIsFilteredOut = OutlinerExtension->IsFilteredOut();

				if (!bIsFilteredOut)
				{
					AccumulatedItemTop -= OutlinerExtension->GetOutlinerSizing().GetTotalHeight();
				}

				if (bIsFilteredOut || !bIsExpanded)
				{
					It.IgnoreCurrentChildren();
				}
			}
		}

		if (OutlinerItem == ParentItem)
		{
			break;
		}

		StopAtChild = ParentItem;
		ParentItem = ParentItem->GetParent();
		ensureMsgf(ParentItem, TEXT("We reached the root item without finding the one that owns this track lane!"));
	}

	SetVerticalPosition(AccumulatedItemTop);

	if (ParentLane)
	{
		ParentLane->PositionParentTrackLanes(GetOutlinerItem(), AccumulatedItemTop);
	}
}

FReply STrackLane::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	static float HitThreshold = 5.f;

	TViewModelPtr<IResizableExtension> ResizableExtension = WeakOutlinerItem.ImplicitPin();

	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && ResizableExtension)
	{
		FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		if (LocalPos.Y >= MyGeometry.GetLocalSize().Y - HitThreshold)
		{
			if (ResizableExtension->IsResizable())
			{
				TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerItem.Pin();
				const float OriginalHeight = OutlinerItem ? OutlinerItem->GetOutlinerSizing().GetTotalHeight() : 10.f;

				DragParameters = FDragParameters(OriginalHeight, LocalPos.Y);
				return FReply::Handled().CaptureMouse(AsShared());
			}
		}
	}

	return FReply::Unhandled();
}

FReply STrackLane::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (DragParameters.IsSet() && HasMouseCapture())
	{
		DragParameters.Reset();
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply STrackLane::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TViewModelPtr<IResizableExtension> ResizableExtension = WeakOutlinerItem.ImplicitPin();

	if (DragParameters.IsSet() && HasMouseCapture() && ResizableExtension)
	{
		FVector2D LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

		float NewHeight = DragParameters->OriginalHeight + (LocalPos.Y - DragParameters->DragStartY);

		if (FMath::RoundToInt(NewHeight) != FMath::RoundToInt(DragParameters->OriginalHeight))
		{
			ResizableExtension->Resize(NewHeight);
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FCursorReply STrackLane::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	static float HitThreshold = 5.f;

	TViewModelPtr<IResizableExtension> ResizableExtension = WeakOutlinerItem.ImplicitPin();

	FVector2D LocalPos = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition());
	if ((LocalPos.Y >= MyGeometry.GetLocalSize().Y - HitThreshold) && ResizableExtension)
	{
		if (ResizableExtension->IsResizable())
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
		}
	}

	return FCursorReply::Unhandled();
}

} // namespace Sequencer
} // namespace UE

