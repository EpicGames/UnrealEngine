// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/ViewUtilities.h"

#include "Styling/AppStyle.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/EditorSharedViewModelData.h"
#include "MVVM/Extensions/IHoveredExtension.h"
#include "MVVM/Views/OutlinerColumns/SOutlinerColumnButton.h"


namespace UE::Sequencer
{

TSharedRef<SWidget> MakeButton(FText HoverText, const FSlateBrush* Image, const FOnClicked& HandleClicked, const FViewModelPtr& ViewModel)
{
	TAttribute<bool> IsHovered;
	if (TSharedPtr<IHoveredExtension> Hoverable = ViewModel.ImplicitCast())
	{
		IsHovered = MakeAttributeSP(Hoverable.ToSharedRef(), &IHoveredExtension::IsHovered);
	}

	TSharedPtr<FSharedViewModelData>        SharedData       = ViewModel->GetSharedData();
	TSharedPtr<FEditorSharedViewModelData>  SharedEditorData = SharedData       ? SharedData->CastThisShared<FEditorSharedViewModelData>() : nullptr;
	TSharedPtr<FEditorViewModel>            Editor           = SharedEditorData ? SharedEditorData->GetEditor() : nullptr;

	TAttribute<bool> IsEnabled;
	if (Editor)
	{
		IsEnabled = MakeAttributeSP(Editor.ToSharedRef(), &FEditorViewModel::IsEditable);
	}

	return SNew(SOutlinerColumnButton)
		.ToolTipText(HoverText)
		.OnClicked(HandleClicked)
		.IsEnabled(IsEnabled)
		.IsRowHovered(IsHovered)
		.Image(Image);
}

TSharedRef<SWidget> MakeButton(FText HoverText, const FSlateBrush* Image, const FOnGetContent& HandleGetMenuContent, const FViewModelPtr& ViewModel)
{
	TAttribute<bool> IsHovered;
	if (TSharedPtr<IHoveredExtension> Hoverable = ViewModel.ImplicitCast())
	{
		IsHovered = MakeAttributeSP(Hoverable.ToSharedRef(), &IHoveredExtension::IsHovered);
	}

	TSharedPtr<FSharedViewModelData>        SharedData       = ViewModel->GetSharedData();
	TSharedPtr<FEditorSharedViewModelData>  SharedEditorData = SharedData       ? SharedData->CastThisShared<FEditorSharedViewModelData>() : nullptr;
	TSharedPtr<FEditorViewModel>            Editor           = SharedEditorData ? SharedEditorData->GetEditor() : nullptr;

	TAttribute<bool> IsEnabled;
	if (Editor)
	{
		IsEnabled = MakeAttributeSP(Editor.ToSharedRef(), &FEditorViewModel::IsEditable);
	}

	return SNew(SOutlinerColumnButton)
		.ToolTipText(HoverText)
		.OnGetMenuContent(HandleGetMenuContent)
		.IsEnabled(IsEnabled)
		.IsRowHovered(IsHovered)
		.Image(Image);
}

TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnClicked& HandleClicked, const FViewModelPtr& ViewModel)
{
	return MakeButton(HoverText, FAppStyle::GetBrush("Sequencer.Outliner.Plus"), HandleClicked, ViewModel);
}

TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnGetContent& HandleGetMenuContent, const FViewModelPtr& ViewModel)
{
	return MakeButton(HoverText, FAppStyle::GetBrush("Sequencer.Outliner.Plus"), HandleGetMenuContent, ViewModel);
}

TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnGetContent& MenuContent, const TAttribute<bool>& HoverState, const TAttribute<bool>& IsEnabled)
{
	return SNew(SOutlinerColumnButton)
		.ToolTipText(HoverText)
		.OnGetMenuContent(MenuContent)
		.IsEnabled(IsEnabled)
		.IsRowHovered(HoverState)
		.Image(FAppStyle::GetBrush("Sequencer.Outliner.Plus"));
}

TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnClicked& OnClicked, const TAttribute<bool>& HoverState, const TAttribute<bool>& IsEnabled)
{
	return SNew(SOutlinerColumnButton)
		.ToolTipText(HoverText)
		.OnClicked(OnClicked)
		.IsEnabled(IsEnabled)
		.IsRowHovered(HoverState)
		.Image(FAppStyle::GetBrush("Sequencer.Outliner.Plus"));
}

} // namespace UE::Sequencer