// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scripting/OutlinerScriptingObject.h"
#include "Scripting/ViewModelScriptingStruct.h"

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModelPtr.h"

#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"
#include "MVVM/Extensions/ISoloableExtension.h"

#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"

#include "SequencerCoreLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerScriptingObject)

#define LOCTEXT_NAMESPACE "SequencerOutlinerScriptingObject"

void USequencerOutlinerScriptingObject::Initialize(UE::Sequencer::TViewModelPtr<UE::Sequencer::FOutlinerViewModel> InOutliner)
{
	using namespace UE::Sequencer;

	InOutliner->GetEditor()->GetSelection()->GetOutlinerSelection()->OnChanged.AddUObject(this, &USequencerOutlinerScriptingObject::BroadcastSelectionChanged);
	WeakOutliner = InOutliner;
}

void USequencerOutlinerScriptingObject::BroadcastSelectionChanged()
{
	OnSelectionChanged.Broadcast();
}

FSequencerViewModelScriptingStruct USequencerOutlinerScriptingObject::GetRootNode() const
{
	using namespace UE::Sequencer;
	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return FSequencerViewModelScriptingStruct();
	}

	return FSequencerViewModelScriptingStruct( Outliner->GetRootItem() );
}

TArray<FSequencerViewModelScriptingStruct> USequencerOutlinerScriptingObject::GetChildren(FSequencerViewModelScriptingStruct InViewModel, FName TypeName) const
{
	using namespace UE::Sequencer;

	FViewModelPtr ViewModel = InViewModel.WeakViewModel.Pin();
	if (!ViewModel)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("ViewModelNotValid", "View Model is no longer valid.").ToString(), ELogVerbosity::Error);
		return TArray<FSequencerViewModelScriptingStruct>();
	}

	TArray<FSequencerViewModelScriptingStruct> ViewModels;

	const FCastableTypeTable* TypeInfo = nullptr;
	if (TypeName != NAME_None)
	{
		TypeInfo = FCastableTypeTable::FindTypeByName(TypeName);
		if (!TypeInfo)
		{
			FFrame::KismetExecutionMessage(*FText::Format(LOCTEXT("InvalidTypeName", "Invalid type name {0} specified."), FText::FromName(TypeName)).ToString(), ELogVerbosity::Error);
			return TArray<FSequencerViewModelScriptingStruct>();
		}
	}

	for (FViewModelPtr Child : ViewModel->GetChildren(EViewModelListType::Outliner))
	{
		if (!TypeInfo || Child->GetTypeTable().Cast(Child.Get(), TypeInfo->GetTypeID()))
		{
			ViewModels.Add(Child);
		}
	}
	return ViewModels;
}

TArray<FSequencerViewModelScriptingStruct> USequencerOutlinerScriptingObject::GetSelection() const
{
	using namespace UE::Sequencer;

	TArray<FSequencerViewModelScriptingStruct> SelectedNodes;

	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return SelectedNodes;
	}

	TSharedPtr<FOutlinerSelection> OutlinerSelection = Outliner->GetEditor()->GetSelection()->GetOutlinerSelection();
	if (OutlinerSelection)
	{
		SelectedNodes.Reserve(OutlinerSelection->Num());
		for (TViewModelPtr<IOutlinerExtension> OutlinerItem : *OutlinerSelection)
		{
			SelectedNodes.Emplace(OutlinerItem);
		}
	}
	return SelectedNodes;
}

void USequencerOutlinerScriptingObject::SetSelection(const TArray<FSequencerViewModelScriptingStruct>& InSelection)
{
	using namespace UE::Sequencer;

	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return;
	}

	TSharedPtr<FSequencerCoreSelection> Selection         = Outliner->GetEditor()->GetSelection();
	TSharedPtr<FOutlinerSelection>      OutlinerSelection = Selection->GetOutlinerSelection();

	if (OutlinerSelection)
	{
		FSelectionEventSuppressor Suppressor = Selection->SuppressEvents();

		OutlinerSelection->Empty();
		for (const FSequencerViewModelScriptingStruct& Item : InSelection)
		{
			TViewModelPtr<IOutlinerExtension> OutlinerItem = Item.WeakViewModel.ImplicitPin();
			if (OutlinerItem)
			{
				TViewModelPtr<ISelectableExtension> Selectable = Item.WeakViewModel.ImplicitPin();
				if (Selectable && !EnumHasAnyFlags(Selectable->IsSelectable(), ESelectionIntent::PersistentSelection))
				{
					UE_LOGF(LogSequencerCore, Warning, "Refusing to select item %ls because it is not selectable.", *OutlinerItem->GetLabel().ToString());
					continue;
				}

				OutlinerSelection->Select(OutlinerItem);
			}
		}
	}
}

TArray<FSequencerViewModelScriptingStruct> USequencerOutlinerScriptingObject::GetMuteNodes()
{
	TArray<FSequencerViewModelScriptingStruct> MutedNodes;

	using namespace UE::Sequencer;
	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return MutedNodes;
	}

	for (const TViewModelPtr<IMutableExtension>& Extension : Outliner->GetRootItem()->GetDescendantsOfType<IMutableExtension>(true))
	{
		if (Extension.IsValid() && Extension->IsMuted())
		{
			MutedNodes.Add(FSequencerViewModelScriptingStruct(Extension.AsModel()));
		}
	}

	return MutedNodes;
}

void USequencerOutlinerScriptingObject::SetMuteNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInMuted)
{
	using namespace UE::Sequencer;

	for (const FSequencerViewModelScriptingStruct& Item : InNodes)
	{
		TViewModelPtr<IMutableExtension> MutableItem = Item.WeakViewModel.ImplicitPin();
		if (MutableItem)
		{
			MutableItem->SetIsMuted(bInMuted);
		}
	}

	if (TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin())
	{
		Outliner->RequestUpdate();
	}
}

TArray<FSequencerViewModelScriptingStruct> USequencerOutlinerScriptingObject::GetSoloNodes()
{
	TArray<FSequencerViewModelScriptingStruct> SoloNodes;

	using namespace UE::Sequencer;
	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return SoloNodes;
	}

	for (const TViewModelPtr<ISoloableExtension>& Extension : Outliner->GetRootItem()->GetDescendantsOfType<ISoloableExtension>(true))
	{
		if (Extension.IsValid() && Extension->IsSolo())
		{
			SoloNodes.Add(FSequencerViewModelScriptingStruct(Extension.AsModel()));
		}
	}

	return SoloNodes;
}

void USequencerOutlinerScriptingObject::SetSoloNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInSoloed)
{
	using namespace UE::Sequencer;

	for (const FSequencerViewModelScriptingStruct& Item : InNodes)
	{
		TViewModelPtr<ISoloableExtension> SoloableItem = Item.WeakViewModel.ImplicitPin();
		if (SoloableItem)
		{
			SoloableItem->SetIsSoloed(bInSoloed);
		}
	}

	if (TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin())
	{
		Outliner->RequestUpdate();
	}
}

TArray<FSequencerViewModelScriptingStruct> USequencerOutlinerScriptingObject::GetDeactivatedNodes()
{
	TArray<FSequencerViewModelScriptingStruct> DeactivatedNodes;

	using namespace UE::Sequencer;
	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return DeactivatedNodes;
	}

	for (const TViewModelPtr<IDeactivatableExtension>& Extension : Outliner->GetRootItem()->GetDescendantsOfType<IDeactivatableExtension>(true))
	{
		if (Extension.IsValid() && Extension->IsDeactivated())
		{
			DeactivatedNodes.Add(FSequencerViewModelScriptingStruct(Extension.AsModel()));
		}
	}

	return DeactivatedNodes;
}

void USequencerOutlinerScriptingObject::SetDeactivatedNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInDeactivated)
{
	using namespace UE::Sequencer;

	for (const FSequencerViewModelScriptingStruct& Item : InNodes)
	{
		TViewModelPtr<IDeactivatableExtension> DeactivatableItem = Item.WeakViewModel.ImplicitPin();
		if (DeactivatableItem)
		{
			DeactivatableItem->SetIsDeactivated(bInDeactivated);
		}
	}
}

TArray<FSequencerViewModelScriptingStruct> USequencerOutlinerScriptingObject::GetLockedNodes()
{
	TArray<FSequencerViewModelScriptingStruct> LockedNodes;

	using namespace UE::Sequencer;
	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return LockedNodes;
	}

	for (const TViewModelPtr<ILockableExtension>& Extension : Outliner->GetRootItem()->GetDescendantsOfType<ILockableExtension>(true))
	{
		if (Extension.IsValid() && Extension->GetLockState() == ELockableLockState::Locked)
		{
			LockedNodes.Add(FSequencerViewModelScriptingStruct(Extension.AsModel()));
		}
	}

	return LockedNodes;
}

void USequencerOutlinerScriptingObject::SetLockedNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInLocked)
{
	using namespace UE::Sequencer;

	for (const FSequencerViewModelScriptingStruct& Item : InNodes)
	{
		TViewModelPtr<ILockableExtension> LockableItem = Item.WeakViewModel.ImplicitPin();
		if (LockableItem)
		{
			LockableItem->SetIsLocked(bInLocked);
		}
	}
}

TArray<FSequencerViewModelScriptingStruct> USequencerOutlinerScriptingObject::GetPinnedNodes()
{
	TArray<FSequencerViewModelScriptingStruct> PinnedNodes;

	using namespace UE::Sequencer;
	TViewModelPtr<FOutlinerViewModel> Outliner = WeakOutliner.Pin();
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return PinnedNodes;
	}

	for (const TViewModelPtr<IPinnableExtension>& Extension : Outliner->GetRootItem()->GetDescendantsOfType<IPinnableExtension>(true))
	{
		if (Extension.IsValid() && Extension->IsPinned())
		{
			PinnedNodes.Add(FSequencerViewModelScriptingStruct(Extension.AsModel()));
		}
	}

	return PinnedNodes;
}

void USequencerOutlinerScriptingObject::SetPinnedNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInPinned)
{
	using namespace UE::Sequencer;

	for (const FSequencerViewModelScriptingStruct& Item : InNodes)
	{
		TViewModelPtr<IPinnableExtension> PinnableItem = Item.WeakViewModel.ImplicitPin();
		if (PinnableItem)
		{
			PinnableItem->SetPinned(bInPinned);
		}
	}
}

void USequencerOutlinerScriptingObject::SetExpansion(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bExpanded)
{
	using namespace UE::Sequencer;

	for (const FSequencerViewModelScriptingStruct& Item : InNodes)
	{
		TViewModelPtr<IOutlinerExtension> OutlinerItem = Item.WeakViewModel.ImplicitPin();
		if (OutlinerItem)
		{
			OutlinerItem->SetExpansion(bExpanded);
		}
	}
}

bool USequencerOutlinerScriptingObject::IsExpanded(FSequencerViewModelScriptingStruct InNode) const
{
	using namespace UE::Sequencer;

	TViewModelPtr<IOutlinerExtension> OutlinerItem = InNode.WeakViewModel.ImplicitPin();
	if (OutlinerItem)
	{
		return OutlinerItem->IsExpanded();
	}

	FFrame::KismetExecutionMessage(*LOCTEXT("ViewModelNotValid", "View Model is no longer valid.").ToString(), ELogVerbosity::Error);
	return false;
}

#undef LOCTEXT_NAMESPACE
