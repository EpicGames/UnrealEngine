// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scripting/SequencerScriptingLayer.h"
#include "Scripting/OutlinerScriptingObject.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/Selection/SequencerCoreSelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerScriptingLayer)


void USequencerScriptingLayer::Initialize(TSharedPtr<UE::Sequencer::FEditorViewModel> InViewModel)
{
	Outliner = NewObject<USequencerOutlinerScriptingObject>(this, "Outliner");
	Outliner->Initialize(InViewModel->GetOutliner());
}
