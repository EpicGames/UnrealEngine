// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scripting/ViewModelScriptingStruct.h"
#include "MVVM/Extensions/IOutlinerExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewModelScriptingStruct)

FSequencerViewModelScriptingStruct::FSequencerViewModelScriptingStruct()
{
	Type = "null";
}

FSequencerViewModelScriptingStruct::FSequencerViewModelScriptingStruct(UE::Sequencer::FViewModelPtr InViewModel)
{
	if (InViewModel)
	{
		WeakViewModel = InViewModel;
		Type = InViewModel->GetTypeTable().GetTypeName();
	}
	else
	{
		Type = "null";
	}
}

FString USequencerViewModelStructExtensions::GetLabel(const FSequencerViewModelScriptingStruct& ViewModel)
{
	using namespace UE::Sequencer;

	TViewModelPtr<IOutlinerExtension> Outliner = ViewModel.WeakViewModel.ImplicitPin();
	return Outliner ? Outliner->GetLabel().ToString() : FString();
}

FString USequencerViewModelStructExtensions::GetPath(const FSequencerViewModelScriptingStruct& ViewModel)
{
	using namespace UE::Sequencer;

	TViewModelPtr<IOutlinerExtension> Outliner = ViewModel.WeakViewModel.ImplicitPin();
	return Outliner ? Outliner->GetPathName(Outliner.AsModel()) : FString();
}
