// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencerCoreModule.h"
#include "Modules/ModuleManager.h"
#include "SequencerCoreLog.h"
#include "MVVM/Extensions/IObjectModelExtension.h"
#include "UObject/ObjectKey.h"
#include "UObject/Class.h"

namespace UE
{
namespace Sequencer
{


/**
 * Interface for the Sequencer module.
 */
class FSequencerCoreModule
	: public ISequencerCoreModule
{
	virtual FDelegateHandle RegisterModelType(const TWeakObjectPtr<UClass>& WeakClass, TFunction<TSharedPtr<FViewModel>(UObject*)> FactoryFunctor) override
	{
		TObjectKey<UClass> Key = WeakClass.Get();
		ensureMsgf(!ModelFactories.Contains(Key), TEXT("Attempting to register a duplicate model factory type"));

		FDelegateHandle Handle(FDelegateHandle::GenerateNewHandle);
		
		ModelFactories.Add(Key, FFactoryEntry{ FactoryFunctor, Handle });
		return Handle;
	}

	virtual void UnRegisterModelType(FDelegateHandle Handle) override
	{
		for (auto It = ModelFactories.CreateIterator(); It; ++It)
		{
			if (It->Value.Handle == Handle)
			{
				It.RemoveCurrent();
				return;
			}
		}
	}

	virtual TSharedPtr<FViewModel> FactoryNewModel(UObject* Object) const override
	{
		check(Object);

		UClass* Class = Object->GetClass();
		while (Class && Class != UObject::StaticClass())
		{
			if (const FFactoryEntry* Factory = ModelFactories.Find(Class))
			{
				TSharedPtr<FViewModel> NewModel = Factory->Functor(Object);
				if (NewModel)
				{
					return NewModel;
				}
			}

			Class = Class->GetSuperClass();
		}

		return nullptr;
	}

private:

	struct FFactoryEntry
	{
		TFunction<TSharedPtr<FViewModel>(UObject*)> Functor;
		FDelegateHandle Handle;
	};

	TMap<TObjectKey<UClass>, FFactoryEntry> ModelFactories;
};


void ISequencerCoreModule::InitializeObjectModel(const FViewModelPtr& ViewModel, UObject* Object)
{
	if (TViewModelPtr<IObjectModelExtension> ObjectModel = ViewModel.ImplicitCast())
	{
		ObjectModel->InitializeObject(Object);
	}
}


} // namespace Sequencer
} // namespace UE


DEFINE_LOG_CATEGORY(LogSequencerCore);
IMPLEMENT_MODULE(UE::Sequencer::FSequencerCoreModule, SequencerCore);
