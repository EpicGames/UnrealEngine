// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeToPixel.h"


namespace UE::Sequencer
{



FTimeToPixelSpace::FTimeToPixelSpace( const FGeometry& AllottedGeometry, const TRange<double>& InLocalViewRange, const FFrameRate& InTickResolution )
	: FTimeToPixelSpace(AllottedGeometry.GetLocalSize().X, InLocalViewRange, InTickResolution)
{}

FTimeToPixelSpace::FTimeToPixelSpace( float WidthPx, const TRange<double>& InLocalViewRange, const FFrameRate& InTickResolution )
	: TickResolution( InTickResolution )
{
	ensureMsgf(WidthPx >= 0, TEXT("Got negative pixel width!"));
	const double VisibleWidth = InLocalViewRange.Size<double>();

	const float MaxPixelsPerSecond = 1000.f;
	PixelsPerSecond = VisibleWidth > 0 ? static_cast<float>(WidthPx / VisibleWidth) : MaxPixelsPerSecond;
	PixelOffset     = -InLocalViewRange.GetLowerBoundValue() * PixelsPerSecond;
}

FTimeToPixelSpace FTimeToPixelSpace::CreateNonLinear(TSharedPtr<INonLinearTimeTransform> InNonLinearTransform) const
{
	FTimeToPixelSpace Copy = *this;
	Copy.NonLinearTransform = InNonLinearTransform;
	return Copy;
}

float FTimeToPixelSpace::SecondsToPixel( double Time ) const
{
	if (NonLinearTransform)
	{
		Time = NonLinearTransform->SourceToView(Time);
	}

	return static_cast<float>(Time * PixelsPerSecond) + PixelOffset;
}


float FTimeToPixelSpace::SecondsDeltaToPixel( double TimeDelta ) const
{
	return static_cast<float>(TimeDelta * PixelsPerSecond);
}


double FTimeToPixelSpace::PixelToSeconds( float PixelX ) const
{
	PixelX -= PixelOffset;
	double Result = (PixelX/PixelsPerSecond);
	if (NonLinearTransform)
	{
		Result = NonLinearTransform->ViewToSource(Result);
	}
	return Result;
}


float FTimeToPixelSpace::FrameToPixel( const FFrameTime& Time ) const
{
	return SecondsToPixel(Time / TickResolution);
}


float FTimeToPixelSpace::FrameDeltaToPixel( const FFrameTime& TimeDelta) const
{
	return static_cast<float>((TimeDelta / TickResolution) * PixelsPerSecond);
}


FFrameTime FTimeToPixelSpace::PixelToFrame( float PixelX ) const
{
	return PixelToSeconds(PixelX) * TickResolution;
}


FFrameTime FTimeToPixelSpace::PixelDeltaToFrame( float PixelDelta ) const
{
	return ( PixelDelta / PixelsPerSecond ) * TickResolution;
}


double FTimeToPixelSpace::PixelDeltaToSeconds( float PixelDelta ) const
{
	return ( PixelDelta / PixelsPerSecond );
}


FFrameRate FTimeToPixelSpace::GetTickResolution() const
{
	return TickResolution;
}


FTimeToPixelSpace FTimeToPixelSpace::RelativeTo(const FFrameTime& FrameZero) const
{
	FTimeToPixelSpace Copy = *this;

	double ViewRangeStart = FrameZero / TickResolution;
	if (Copy.NonLinearTransform)
	{
		ViewRangeStart = Copy.NonLinearTransform->SourceToView(ViewRangeStart);
	}

	Copy.PixelOffset = -ViewRangeStart*PixelsPerSecond;
	return Copy;
}


} // namespace UE::Sequencer

FTimeToPixel FTimeToPixel::RelativeTo(const FFrameTime& FrameZero) const
{
	FTimeToPixel Copy = *this;
	double ViewRangeStart = FrameZero / TickResolution;
	if (Copy.NonLinearTransform)
	{
		ViewRangeStart = Copy.NonLinearTransform->SourceToView(ViewRangeStart);
	}

	Copy.PixelOffset = -ViewRangeStart*PixelsPerSecond;
	return Copy;
}