// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "MVVM/ViewModelPtr.h"
#include "Templates/Function.h"


namespace UE::Sequencer
{

class FViewModel;

/**
 * Interface for the Sequencer module.
 */
class ISequencerCoreModule
	: public IModuleInterface
{
public:

	/**
	 * Register a new MVVM view model type factory functor that will be used for any UObjects of the specified class type
	 * @note It is a violation to attempt to register multiple functors for the same class type
	 * 
	 * @param WeakClass       A weak pointer to the class type that should invoke this factory
	 * @param FactoryFunctor  The functor that defines how to create a view model for the object
	 * 
	 * @return A delegate handle that can be passed to UnRegisterModelType to unregister this factory
	 */
	virtual FDelegateHandle RegisterModelType(const TWeakObjectPtr<UClass>& WeakClass, TFunction<TSharedPtr<FViewModel>(UObject*)> FactoryFunctor) = 0;


	/**
	 * Unregister a previously registered view model type
	 *
	 * @param Handle  The handle to the factory functor that was provided from calling RegisterModelType
	 */
	virtual void UnRegisterModelType(FDelegateHandle Handle) = 0;


	/**
	 * Attempt to create a new model type for the specified object
	 *
	 * @param Object  The object to create a view model for
	 * @return A pointer to a new view-model, or nullptr if the object didn't match any registered class.
	 */
	virtual TSharedPtr<FViewModel> FactoryNewModel(UObject* Object) const = 0;

public:

	/**
	 * Register a new MVVM view model type from a view model type ID that will be used for any UObjects of the specified class type
	 *
	 * @param Object  The object to create a view model for
	 * @param Type    The view model type ID to create for UObjects of type WeakClass
	 *
	 * @return A delegate handle that can be passed to UnRegisterModelType to unregister this factory
	 */
	template<typename T>
	FDelegateHandle RegisterModelType(const TWeakObjectPtr<UClass>& WeakClass, TViewModelTypeID<T> Type)
	{
		auto Factory = [this, Type](UObject* Object)
		{
			TSharedPtr<T> NewViewModel = MakeShared<T>();

			this->InitializeObjectModel(NewViewModel, Object);

			return NewViewModel;
		};
		return RegisterModelType(WeakClass, Factory);
	}

private:


	SEQUENCERCORE_API void InitializeObjectModel(const FViewModelPtr& ViewModel, UObject* Object);
};


} // namespace UE::Sequencer