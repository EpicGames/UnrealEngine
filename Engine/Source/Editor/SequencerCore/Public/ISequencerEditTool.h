// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Input/CursorReply.h"
#include "ISequencerInputHandler.h"

class FSlateWindowElementList;
class SWidget;

namespace UE
{
namespace Sequencer
{

struct ITrackAreaHotspot;
class FVirtualTrackArea;
class FViewModel;

/**
 * Interface for drag and drop operations that are handled by edit tools in Sequencer.
 */
class ISequencerEditToolDragOperation
{
public:

	/**
	 * Called to initiate a drag
	 *
	 * @param MouseEvent		The associated mouse event for dragging
	 * @param LocalMousePos		The current location of the mouse, relative to the top-left corner of the physical track area
	 * @param VirtualTrackArea	A virtual track area that can be used for pixel->time conversions and hittesting
	 */
	virtual void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) = 0;

	/**
	 * Notification called when the mouse moves while dragging
	 *
	 * @param MouseEvent		The associated mouse event for dragging
	 * @param LocalMousePos		The current location of the mouse, relative to the top-left corner of the physical track area
	 * @param VirtualTrackArea	A virtual track area that can be used for pixel->time conversions and hittesting
	 */
	virtual void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) = 0;

	/** Called when a drag has ended
	 *
	 * @param MouseEvent		The associated mouse event for dragging
	 * @param LocalMousePos		The current location of the mouse, relative to the top-left corner of the physical track area
	 * @param VirtualTrackArea	A virtual track area that can be used for pixel->time conversions and hittesting
	 */
	virtual void OnEndDrag( const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) = 0;

	/** Request the cursor for this drag operation */
	virtual FCursorReply GetCursor() const = 0;

	/** Override to implement drag-specific paint logic */
	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const = 0;

	/** Get the track model that owns this drag operation (for hover suppression). Returns nullptr if not applicable. */
	virtual TSharedPtr<FViewModel> GetDragOwnerTrackModel() const { return nullptr; }

public:

	/** Virtual destructor. */
	virtual ~ISequencerEditToolDragOperation() = default;
};


/**
 * Interface for edit tools in Sequencer.
 */
class ISequencerEditTool : public ISequencerInputHandler
{
public:
	virtual ~ISequencerEditTool() override = default;

	UE_DEPRECATED(5.8, "OnMouseCaptureLost has been deprecated. Please override OnMouseCaptureLost from ISequencerInputHandler instead.")
	virtual void OnMouseCaptureLost() {}

	virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const override
	{
		// Redirect to the old 2-argument version so existing overrides still work
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return OnCursorQuery(InGeometry, InPointerEvent);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.8, "OnCursorQuery has been deprecated. Please override OnCursorQuery from ISequencerInputHandler instead.")
	virtual FCursorReply OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const { return FCursorReply::Unhandled(); }

	UE_DEPRECATED(5.8, "OnMouseEnter has been deprecated. Please override OnMouseEnter from ISequencerInputHandler instead.")
	virtual void OnMouseEnter(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) {}

	UE_DEPRECATED(5.8, "OnMouseLeave has been deprecated. Please override OnMouseLeave from ISequencerInputHandler instead.")
	virtual void OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& InPointerEvent) {}

	UE_DEPRECATED(5.8, "OnKeyDown has been deprecated. Please override OnKeyDown from ISequencerInputHandler instead.")
	virtual FReply OnKeyDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) { return FReply::Unhandled(); }

	UE_DEPRECATED(5.8, "OnKeyUp has been deprecated. Please override OnKeyUp from ISequencerInputHandler instead.")
	virtual FReply OnKeyUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) { return FReply::Unhandled(); }

	virtual FName GetIdentifier() const = 0;

	virtual bool CanDeactivate() const { return true; }

	virtual int32 OnPaint(const FGeometry& InGeometry, const FSlateRect& InCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const = 0;

	virtual TSharedPtr<ITrackAreaHotspot> GetDragHotspot() const { return nullptr; }
	virtual ISequencerEditToolDragOperation* GetCurrentDragOperation() const { return nullptr; }
};

} // namespace Sequencer
} // namespace UE

