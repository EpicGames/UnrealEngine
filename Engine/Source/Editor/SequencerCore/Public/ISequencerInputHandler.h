// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Input/Reply.h"

class SWidget;

/**
 * Common base-class for objects that handle input in the sequencer.
 */
struct ISequencerInputHandler
{
	virtual ~ISequencerInputHandler() = default;

	virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const { return FCursorReply::Unhandled(); }

	virtual void OnMouseEnter(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) {}
	virtual void OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& InPointerEvent) {}
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) { return FReply::Unhandled(); }
	virtual void OnMouseCaptureLost() {}

	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) { return FReply::Unhandled(); }
	virtual FReply OnMouseWheel(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) { return FReply::Unhandled(); }

	virtual FReply OnKeyDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) { return FReply::Unhandled(); }
	virtual FReply OnKeyUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) { return FReply::Unhandled(); }

	virtual void OnFocusLost(SWidget& OwnerWidget, const FFocusEvent& InFocusEvent) {}
};
