// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/SparseBitSet.h"
#include "UObject/NameTypes.h"

#include <type_traits>

namespace UE::Sequencer
{

template<typename T>
struct TViewModelTypeID;

template<typename ...T>
struct TImplements;

class ICastable;
class IDynamicExtension;

struct FCastableTypeTable
{
	explicit FCastableTypeTable(uint8* InTypeMaskStorage);

	FCastableTypeTable(const FCastableTypeTable&) = delete;
	void operator=(const FCastableTypeTable&) = delete;
	FCastableTypeTable(FCastableTypeTable&&) = delete;
	void operator=(FCastableTypeTable&&) = delete;

	/**
	 * Make a new type table for the templated type, ID and typename.
	 * The new table is allocated using malloc and is intentionally leaked
	 */
	template<typename T>
	static FCastableTypeTable* MakeTypeTable(const void* Unused, uint32 TypeID, FName InName)
	{
		return nullptr;
	}
	template<typename T>
	static FCastableTypeTable* MakeTypeTable(const ICastable* Unused, uint32 TypeID, FName InName)
	{
		FCastableTypeTableGenerator Generator;
		Generator.Generate<T>();
		return Generator.Commit(TypeID, InName);
	}
	template<typename T>
	static FCastableTypeTable* MakeTypeTable(const IDynamicExtension* Unused, uint32 TypeID, FName InName)
	{
		FCastableTypeTableGenerator Generator;
		Generator.Generate<T>();
		return Generator.Commit(TypeID, InName);
	}

	/**
	 * Attempt to locate a type table by its C++ name
	 */
	SEQUENCERCORE_API static const FCastableTypeTable* FindTypeByName(FName InName);


	/**
	 * Use this type table to cast the specified base ptr to another type
	 * @note BasePtr must be an instance of the type specified by this type table, or the result is undefined.
	 *
	 * @return BasePtr casted to the type ID specified by ToType, or nullptr if the cast failed
	 */
	const void* Cast(const ICastable* BasePtr, uint32 ToType) const
	{
		return CastImpl(BasePtr, ToType);
	}


	/**
	 * Use this type table to cast the specified base ptr to another type
	 * @note BasePtr must be an instance of the type specified by this type table, or the result is undefined.
	 *
	 * @return BasePtr casted to the type ID specified by ToType, or nullptr if the cast failed
	 */
	const void* Cast(const IDynamicExtension* BasePtr, uint32 ToType) const
	{
		return CastImpl(BasePtr, ToType);
	}

	/**
	 * Retrieve the C++ type name that this type table relates to
	 */
	FName GetTypeName() const
	{
		return ThisTypeName;
	}


	/**
	 * Retrieve the type ID for this type table.
	 * Warning: This type ID is not deterministic or stable across different processes and should not be saved persistently
	 */
	uint32 GetTypeID() const
	{
		return ThisTypeID;
	}

private:

	/**
	 * Cast implementation function
	 */
	const void* CastImpl(const void* BasePtr, uint32 ToType) const
	{
		const int32 SparseTypeIndex = TypeMask.GetSparseBucketIndex(ToType);
		if (SparseTypeIndex != INDEX_NONE)
		{
			return static_cast<const uint8*>(BasePtr) + StaticTypeOffsets[SparseTypeIndex];
		}
		return nullptr;
	}

private:

	/** Utility class that generates a type table used for dynamic casting and instance of an ICastable object */ 
	struct FCastableTypeTableGenerator
	{
		/**
		 * Generate the type table from a template type
		 */
		template<typename T>
		void Generate()
		{
			// Compute the start offset from a base ICastable to T
			// This ensures we compute the offsets from the correct point
			const int16 StartOffset = ComputeStartOffset<T>((T*)0);
			AddStaticType<T>((T*)nullptr, StartOffset);
			ConsumeImplements<T>((T*)nullptr, StartOffset);
		}

		/** Commit this generator to its final form */
		SEQUENCERCORE_API FCastableTypeTable* Commit(uint32 TypeID, FName InName);

		/**
		 * Compute the ptr offset from one class to another in an inheritance hierarchy.
		 * @note: This can be negative for situations where multiple inheritance would put
		 *        the FromType class after the ToType in memory.
		 */
		template<typename FromType, typename ToType>
		static int16 ClassOffsetFrom()
		{
			alignas(FromType) uint8 Buffer[sizeof(FromType)];

			FromType* From = reinterpret_cast<FromType*>(Buffer);
			ToType*   To   = static_cast<ToType*>(From);

			const int32 Offset = static_cast<int32>((uint8*)To - (uint8*)From);
			checkf(Offset >= std::numeric_limits<int16>::lowest() && Offset <= std::numeric_limits<int16>::max(), TEXT("Class offset is too large to be supported by ICastable"));
			return static_cast<int16>(Offset);
		}

		/**
		 * Compute the start offset from an ICastable type to the root of T
		 * This offset is added to all other class offsets that are computed from T
		 * so that we can always cast from an ICastable* to the concrete types
		 */
		template<typename T>
		static int16 ComputeStartOffset(const ICastable*)
		{
			return ClassOffsetFrom<ICastable, T>();
		}

		/**
		 * Compute the start offset from an IDynamicExtension type to the root of T
		 * This offset is added to all other class offsets that are computed from T
		 * so that we can always cast from an IDynamicExtension* to the concrete types
		 */
		template<typename T>
		static int16 ComputeStartOffset(const IDynamicExtension*)
		{
			return ClassOffsetFrom<IDynamicExtension, T>();
		}

		uint32 ByteSize() const
		{
			return sizeof(FCastableTypeTable) + sizeof(int16)*StaticTypeOffsets.Num() + sizeof(uint8)*StaticTypeOffsets.Num();
		}

		template<typename StartClass, typename T, typename U = typename T::Implements>
		void ConsumeImplements(T*, int16 StartOffset)
		{
			ConsumeImplementsImpl<StartClass>((typename T::Implements*)nullptr, StartOffset);
		}

		template<typename StartClass>
		void ConsumeImplements(...)
		{
		}

		template<typename StartClass, typename ...Types>
		void ConsumeImplementsImpl(TImplements<Types...>*, int16 StartOffset)
		{
			(AddStaticType<StartClass>((Types*)nullptr, StartOffset), ...);
			(ConsumeImplements<StartClass>((Types*)nullptr, StartOffset), ...);
		}

		template<typename StartClass, typename T>
		void ConsumeImplements(void*, int16 StartOffset)
		{
		}

		template<typename StartClass, typename T, typename U = decltype(T::ID)>
		void AddStaticType(T*, int16 StartOffset)
		{
			using namespace UE::MovieScene;

			TViewModelTypeID<T>& ID = T::ID;

			int16 OffsetFromBase = StartOffset + ClassOffsetFrom<StartClass, T>();
			if (TypeMask.SetBit(ID.GetTypeID()) == ESparseBitSetBitResult::NewlySet)
			{
				int32 SparseIndex = TypeMask.GetSparseBucketIndex(ID.GetTypeID());
				StaticTypeOffsets.Insert(OffsetFromBase, SparseIndex);
			}
		}

		template<typename StartClass>
		constexpr void AddStaticType(void*, int16 StartOffset)
		{
		}

	private:

		/* Accumulator sparse bitset of types that are implemented on the type that Generate was called with */
		MovieScene::TFixedSparseBitSet<uint32> TypeMask;
		/* Typeoffsets from T to the ptr for each type within TypeMask */
		TArray<int16> StaticTypeOffsets;
	};

private:

	/** Ptr to the static type offsets allocated at the end of this struct. Size = NumStaticTypes */
	int16* StaticTypeOffsets;
	/** Fixed sparse bitset of types that this table contains. Storage is allocated at the end of this struct. */
	MovieScene::TFixedSparseBitSet<uint32, MovieScene::TFixedSparseBitSetBucketStorage<uint8>> TypeMask;
	/** The number of static types implemented within this type table. */
	uint32 NumStaticTypes;
	/** Unique static type ID for this type. */
	uint32 ThisTypeID;
	/** The name of the C++ type that this table relates to. */
	FName ThisTypeName;
};

} // namespace UE::Sequencer