// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SortedMap.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"
#include "Misc/InlineValue.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

struct FCastableTypeTable;
class FDynamicExtensionContainer;
class FViewModel;

struct FDynamicExtensionContainerIterator;

template<typename T>
struct TDynamicExtensionContainerIterator;

template<typename T>
struct TDynamicExtensionContainerIteratorProxy;

/**
 * Base class for dynamic extensions that can be added to sequence data models and participate in dynamic casting.
 */
class IDynamicExtension : public TSharedFromThis<IDynamicExtension>
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IDynamicExtension)

	virtual ~IDynamicExtension()= default;

	/** Called when the extension is created on a data model */
	virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) {}
	/** Called after all extensions have been created on a data model */
	virtual void OnPostInitialize() { OnReinitialize(); }
	/** Called to reinitialize extensions after a major change */
	virtual void OnReinitialize() {}
};

DECLARE_DELEGATE_OneParam(FDynamicExtensionCallback, TSharedRef<FViewModel>);


/**
 * Base class for supporting dynamic extensions that participate in dynamic casting.
 */
class FDynamicExtensionContainer
{
public:

	virtual ~FDynamicExtensionContainer() = default;

	FDynamicExtensionContainer() = default;

	FDynamicExtensionContainer(const FDynamicExtensionContainer&) = delete;
	FDynamicExtensionContainer& operator=(const FDynamicExtensionContainer&) = delete;

public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(SEQUENCERCORE_API, FDynamicExtensionContainer)

	template<typename T>
	const T* CastDynamic() const
	{
		const void* Result = CastDynamic(T::ID);
		return static_cast<const T*>(Result);
	}

	template<typename T>
	const T* CastDynamicChecked() const
	{
		const void* Result = CastDynamicChecked(T::ID);
		return static_cast<const T*>(Result);
	}

	template<typename T>
	T* CastDynamic()
	{
		return const_cast<T*>(const_cast<const FDynamicExtensionContainer*>(this)->CastDynamic<T>());
	}

	template<typename T>
	T* CastDynamicChecked()
	{
		return const_cast<T*>(const_cast<const FDynamicExtensionContainer*>(this)->CastDynamicChecked<T>());
	}

	template<typename T>
	TDynamicExtensionContainerIteratorProxy<T> FilterDynamic()
	{
		return TDynamicExtensionContainerIteratorProxy<T>(this);
	}

	SEQUENCERCORE_API const void* CastDynamic(FViewModelTypeID Type) const;
	SEQUENCERCORE_API void RemoveDynamicExtension(FViewModelTypeID Type);

	const void* CastDynamicChecked(FViewModelTypeID Type) const
	{
		const void* Result = CastDynamic(Type);
		check(Result);
		return Result;
	}

	void* CastDynamic(FViewModelTypeID Type)
	{
		return const_cast<void*>(const_cast<const FDynamicExtensionContainer*>(this)->CastDynamic(Type));
	}

	void* CastDynamicChecked(FViewModelTypeID Type)
	{
		void* Result = CastDynamic(Type);
		check(Result);
		return Result;
	}

protected:

	template<typename T, typename... InArgTypes>
	T& AddDynamicExtension(TSharedRef<FViewModel> InOwner, InArgTypes&&... Args)
	{
		T* Existing = CastDynamic<T>();
		if (Existing)
		{
			return static_cast<T&>(*Existing);
		}

		TSharedRef<T> NewExtension = MakeShared<T>(Forward<InArgTypes>(Args)...);
		DynamicExtensions.Emplace(FDynamicExtensionInfo{ T::ID.GetTypeTable(), NewExtension });
		NewExtension->OnCreated(InOwner);
		return NewExtension.Get();
	}

	template<typename T>
	void RemoveDynamicExtension()
	{
		RemoveDynamicExtension(T::ID);
	}

	void PostInitializeExtensions()
	{
		// Iterate by index with null checking to support mutation during iteration
		for (int32 Index = 0; Index < DynamicExtensions.Num(); ++Index)
		{
			if (DynamicExtensions[Index].Extension)
			{
				DynamicExtensions[Index].Extension->OnPostInitialize();
			}
		}
		// Clean up nulls
		for (int32 Index = DynamicExtensions.Num()-1; Index >= 0; --Index)
		{
			if (!DynamicExtensions[Index].Extension)
			{
				DynamicExtensions.RemoveAtSwap(Index, 1);
			}
		}
	}

	void ReinitializeExtensions()
	{
		// Iterate by index with null checking to support mutation during iteration
		for (int32 Index = 0; Index < DynamicExtensions.Num(); ++Index)
		{
			if (DynamicExtensions[Index].Extension)
			{
				DynamicExtensions[Index].Extension->OnReinitialize();
			}
		}
		// Clean up nulls
		for (int32 Index = DynamicExtensions.Num()-1; Index >= 0; --Index)
		{
			if (!DynamicExtensions[Index].Extension)
			{
				DynamicExtensions.RemoveAtSwap(Index, 1);
			}
		}
	}

private:

	friend struct FDynamicExtensionContainerIterator;
	template<typename T> friend struct TDynamicExtensionContainerIteratorProxy;

	struct FDynamicExtensionInfo
	{
		FCastableTypeTable* TypeTable;
		TSharedPtr<IDynamicExtension> Extension;
	};

	TArray<FDynamicExtensionInfo> DynamicExtensions;
};

struct FDynamicExtensionContainerIterator
{
	explicit operator bool() const
	{
		return (bool)Iterator;
	}

	SEQUENCERCORE_API FDynamicExtensionContainerIterator& operator++();

	SEQUENCERCORE_API friend bool operator!=(const FDynamicExtensionContainerIterator& A, const FDynamicExtensionContainerIterator& B);

protected:
	using IteratorType = TArray<FDynamicExtensionContainer::FDynamicExtensionInfo>::TConstIterator;

	friend FDynamicExtensionContainer;

	SEQUENCERCORE_API explicit FDynamicExtensionContainerIterator(IteratorType&& InIterator, FViewModelTypeID InType);

	void* CurrentExtension = nullptr;
	IteratorType Iterator;
	FViewModelTypeID Type;
};

template<typename T>
struct TDynamicExtensionContainerIteratorProxy
{
	FORCEINLINE TDynamicExtensionContainerIterator<T> begin() { return TDynamicExtensionContainerIterator<T>(IteratorType(Container->DynamicExtensions)); }
	FORCEINLINE TDynamicExtensionContainerIterator<T> end()   { return TDynamicExtensionContainerIterator<T>(IteratorType(Container->DynamicExtensions, Container->DynamicExtensions.Num())); }

private:
	friend FDynamicExtensionContainer;

	using IteratorType = TArray<FDynamicExtensionContainer::FDynamicExtensionInfo>::TConstIterator;

	TDynamicExtensionContainerIteratorProxy(FDynamicExtensionContainer* InContainer)
		: Container(InContainer)
	{}

	FDynamicExtensionContainer* Container;
};

template<typename T>
struct TDynamicExtensionContainerIterator : FDynamicExtensionContainerIterator
{
	T& operator*() const
	{
		check(CurrentExtension);
		return *static_cast<T*>(CurrentExtension);
	}

protected:
	friend TDynamicExtensionContainerIteratorProxy<T>;

	explicit TDynamicExtensionContainerIterator(FDynamicExtensionContainerIterator::IteratorType InIterator)
		: FDynamicExtensionContainerIterator(MoveTemp(InIterator), T::ID)
	{}
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
