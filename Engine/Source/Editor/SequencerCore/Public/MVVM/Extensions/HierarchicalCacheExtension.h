// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCoreFwd.h"
#include "MVVM/ViewModelTypeID.h"
#include "Delegates/DelegateCombinations.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/ViewModelPtr.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;

namespace UE::Sequencer
{

class IHierarchicalCache;

template<typename EnumType>
EnumType CombinePropagatedChildFlags(EnumType ParentFlags, EnumType CombinedChildFlags)
{
	return ParentFlags | CombinedChildFlags;
}

DECLARE_MULTICAST_DELEGATE_OneParam(FPreUpdateCachesEvent, FViewModelPtr);

class IHierarchicalCache
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IHierarchicalCache)

	virtual ~IHierarchicalCache()= default;

	virtual void BeginUpdate() = 0;
	virtual void PreVisitChildren(const FViewModelPtr& ViewModel) = 0;
	virtual void PostVisitChildren(const FViewModelPtr& ViewModel) = 0;
	virtual void EndUpdate() = 0;
};


class FHierarchicalCacheExtension
	: public IDynamicExtension
{
public:

	UE_API ~FHierarchicalCacheExtension();

	FPreUpdateCachesEvent PreUpdateCachesEvent;

	UE_API void Initialize(const FViewModelPtr& InRootModel);

	UE_API void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;

	UE_API void OnHierarchyUpdated();

	UE_API void UpdateCachedFlags();
	UE_API void UpdateAllCachedFlags(const FViewModelPtr& ViewModel, TArrayView<IHierarchicalCache* const> HierarchicalCaches);

private:

	UE_API void UpdateCachedFlagsForModel(const FViewModelPtr& ViewModel, TArrayView<IHierarchicalCache* const> HierarchicalCaches);

protected:

	FWeakViewModelPtr WeakOwnerModel;
	FWeakViewModelPtr WeakRootModel;
	EViewModelListType ModelListFilter;
};



template<typename FlagsType>
class TFlagStateCacheExtension
	: public IDynamicExtension
	, public IHierarchicalCache
{
public:

	using Implements = TImplements<IDynamicExtension, IHierarchicalCache>;

	virtual ~TFlagStateCacheExtension() = default;

	FlagsType GetRootFlags() const
	{
		return RootFlags;
	}
	FlagsType GetCachedFlags(const FViewModelPtr& ViewModel) const
	{
		return GetCachedFlags(ViewModel->GetModelID());
	}
	FlagsType GetCachedFlags(uint32 InModelID) const
	{
		return CachedFlagsFromNodeID.FindRef(InModelID);
	}

private:

	virtual FlagsType ComputeFlagsForModel(const FViewModelPtr& ViewModel) = 0;
	virtual void PostComputeChildrenFlags(const FViewModelPtr& ViewModel, FlagsType& OutThisModelFlags, FlagsType& OutPropagateToParentFlags) {}

protected:

	void BeginUpdate() override
	{
		CachedFlagsFromNodeID.Empty();
		IndividualItemFlags.Emplace(FlagsType::None);
		AccumulatedChildFlags.Emplace(FlagsType::None);
	}
	void PreVisitChildren(const FViewModelPtr& ViewModel) override
	{
		FlagsType ThisModelFlags = ComputeFlagsForModel(ViewModel);
		IndividualItemFlags.Emplace(ThisModelFlags);
		AccumulatedChildFlags.Emplace(FlagsType::None);
	}
	void PostVisitChildren(const FViewModelPtr& ViewModel) override
	{
		FlagsType ThisModelFlags              = IndividualItemFlags.Pop();
		FlagsType FlagsPropagatedFromChildren = AccumulatedChildFlags.Pop();

		ThisModelFlags = CombinePropagatedChildFlags(ThisModelFlags, FlagsPropagatedFromChildren);
		PostComputeChildrenFlags(ViewModel, ThisModelFlags, AccumulatedChildFlags.Last());

		AccumulatedChildFlags.Last() |= (ThisModelFlags & FlagsType::InheritedFromChildren);

		if (ThisModelFlags != FlagsType::None)
		{
			CachedFlagsFromNodeID.Add(ViewModel->GetModelID(), ThisModelFlags);
		}
	}
	void EndUpdate() override
	{
		RootFlags = AccumulatedChildFlags.Last();
		AccumulatedChildFlags.Empty();
		IndividualItemFlags.Empty();

		CachedFlagsFromNodeID.Shrink();
	}

protected:

	TArray<FlagsType> AccumulatedChildFlags;
	TArray<FlagsType> IndividualItemFlags;
	TMap<uint32, FlagsType> CachedFlagsFromNodeID;
	FlagsType RootFlags;
};


} // namespace UE::Sequencer

#undef UE_API
