// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "MVVM/Extensions/HierarchicalCacheExtension.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

enum class EDeactivatableState
{
	None,
	Deactivated,
	PartiallyDeactivated,
};

/**
 * An extension for outliner nodes that can be deactivated
 */
class IDeactivatableExtension
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IDeactivatableExtension);

	virtual ~IDeactivatableExtension() = default;

	/** Returns whether this item is deactivated */
	virtual bool IsDeactivated() const = 0;

	/** Set this item's deactivated state */
	virtual void SetIsDeactivated(const bool bInIsDeactivated) = 0;

	/** Returns whether this deactivatable can be deactivated by a parent, and should report its deactivated state to a parent */
	virtual bool IsInheritable() const
	{
		return true;
	}
};

enum class ECachedDeactiveState
{
	None                          = 0,

	Deactivatable                 = 1 << 0,
	DeactivatableChildren         = 1 << 1,
	Deactivated                   = 1 << 2,
	PartiallyDeactivatedChildren  = 1 << 3,
	ImplicitlyDeactivatedByParent = 1 << 4,
	Inheritable                   = 1 << 5,

	InheritedFromChildren = DeactivatableChildren | PartiallyDeactivatedChildren,
};
ENUM_CLASS_FLAGS(ECachedDeactiveState)

SEQUENCERCORE_API ECachedDeactiveState CombinePropagatedChildFlags(const ECachedDeactiveState ParentFlags, ECachedDeactiveState CombinedChildFlags);

class FDeactiveStateCacheExtension
	: public TFlagStateCacheExtension<ECachedDeactiveState>
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FDeactiveStateCacheExtension);

private:
	UE_API ECachedDeactiveState ComputeFlagsForModel(const FViewModelPtr& ViewModel) override;
	UE_API void PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedDeactiveState& OutThisModelFlags, ECachedDeactiveState& OutPropagateToParentFlags) override;
};

} // namespace UE::Sequencer

#undef UE_API
