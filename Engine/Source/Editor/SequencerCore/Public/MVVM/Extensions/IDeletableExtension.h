// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class IDeletableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IDeletableExtension)

	virtual ~IDeletableExtension()= default;

	virtual bool CanDelete(FText* OutErrorMessage) const = 0;
	virtual void Delete() = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
