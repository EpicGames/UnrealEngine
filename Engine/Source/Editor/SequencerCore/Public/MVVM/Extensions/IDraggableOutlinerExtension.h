// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "Misc/FrameNumber.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;

namespace UE
{
namespace Sequencer
{


/**
 * Extension for models that can be dragged
 */
class IDraggableOutlinerExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IDraggableOutlinerExtension)

	virtual ~IDraggableOutlinerExtension()= default;

	/** Returns whether the model can be dragged */
	virtual bool CanDrag() const = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
