// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "Misc/FrameNumber.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;

namespace UE
{
namespace Sequencer
{

class FViewModel;

/**
 * Interface for drag operations
 */
class IDragOperation
{
public:
	virtual ~IDragOperation() = default;

	/** Adds a time that the drag operation will snap to */
	virtual void AddSnapTime(FFrameNumber SnapTime) = 0;

	/** Adds a model to the drag operation to be dragged */
	virtual void AddModel(TSharedPtr<FViewModel> Model) = 0;
};

/**
 * Extension for models that can be dragged
 */
class IDraggableTrackAreaExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IDraggableTrackAreaExtension)

	virtual ~IDraggableTrackAreaExtension()= default;

	/** Returns whether the model can be dragged */
	virtual bool CanDrag() const = 0;
	/** Called at the beginning of the drag operation */
	virtual void OnBeginDrag(IDragOperation& DragOperation) = 0;
	/** Called at the end of a drag operation */
	virtual void OnEndDrag(IDragOperation& DragOperation) = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
