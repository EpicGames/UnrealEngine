// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

class FText;

namespace UE
{
namespace Sequencer
{

class IDragOperation;

/**
 * Extension for models that can be dragged
 */
class IDroppableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IDroppableExtension)

	virtual ~IDroppableExtension()= default;

	/** Check whether this droppable extension is able to accept the specified drag */
	virtual bool CanAcceptDrag(IDragOperation& DragOperation, FText* OutInfo) const = 0;
	virtual void OnHover(IDragOperation& DragOperation) = 0;
	virtual void ProcessDragOperation(IDragOperation& DragOperation) = 0;
	virtual void OnUnhover(IDragOperation& DragOperation) = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
