// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class FViewModel;

/**
 * Virtual bounding box information for track area elements.
 */
struct FVirtualGeometry
{
	FVirtualGeometry()
		: Top(0.f), Height(0.f), NestedBottom(0.f)
	{}

	FVirtualGeometry(float InTop, float InHeight, float InNestedBottom)
		: Top(InTop), Height(InHeight), NestedBottom(InNestedBottom)
	{}

	float GetTop() const
	{
		return Top;
	}

	float GetHeight() const
	{
		return Height;
	}

	float GetNestedHeight() const
	{
		return NestedBottom - Top;
	}

	float GetTotalChildHeight() const
	{
		return NestedBottom - Top - Height;
	}

	float Top;
	float Height;
	float NestedBottom;
};

/**
 * Interface for data models who can store virtual geometry information
 */
class IGeometryExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IGeometryExtension)

	virtual ~IGeometryExtension()= default;

	virtual FVirtualGeometry GetVirtualGeometry() const = 0;
	virtual void ReportVirtualGeometry(const FVirtualGeometry& VirtualGeometry) = 0;

	static UE_API float UpdateVirtualGeometry(float InitialVirtualPosition, TSharedPtr<FViewModel> InInitialItem);
};

/**
 * Basic utility implementation of the IGeometryExtension interface
 */
class FGeometryExtensionShim : public IGeometryExtension
{
public:

	FVirtualGeometry GetVirtualGeometry() const override
	{
		return VirtualGeometry;
	}

	void ReportVirtualGeometry(const FVirtualGeometry& InVirtualGeometry) override
	{
		VirtualGeometry = InVirtualGeometry;
	}

protected:

	FVirtualGeometry VirtualGeometry;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
