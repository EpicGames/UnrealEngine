// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;

namespace UE
{
namespace Sequencer
{

class IGroupableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IGroupableExtension)

	virtual ~IGroupableExtension()= default;

	virtual void GetIdentifierForGrouping(TStringBuilder<128>& OutString) const = 0;

	FString GetIdentifierForGrouping() const
	{
		TStringBuilder<128> Builder;
		GetIdentifierForGrouping(Builder);
		return Builder.ToString();
	}
};


} // namespace Sequencer
} // namespace UE

#undef UE_API
