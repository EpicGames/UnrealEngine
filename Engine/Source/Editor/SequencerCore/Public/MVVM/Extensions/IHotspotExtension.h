// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class IHotspotExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IHotspotExtension)

	virtual ~IHotspotExtension()= default;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
