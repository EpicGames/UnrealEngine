// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class IHoveredExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IHoveredExtension)

	virtual ~IHoveredExtension() = default;

	virtual bool IsHovered() const = 0;

	virtual void OnHovered() = 0;
	virtual void OnUnhovered() = 0;
};

class FHoveredExtensionShim : public IHoveredExtension
{
public:

	virtual bool IsHovered() const { return bIsHovered; }

	virtual void OnHovered() { bIsHovered = true; }
	virtual void OnUnhovered() { bIsHovered = false; }

protected:

	bool bIsHovered = false;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
