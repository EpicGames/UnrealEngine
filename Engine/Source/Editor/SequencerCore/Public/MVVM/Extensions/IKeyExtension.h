// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModelTypeID.h"
#include "Curves/KeyHandle.h"
#include "Misc/FrameTime.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;
struct FPointerEvent;
struct FKeyDrawParams;
struct FGeometry;
class FCurveModel;

namespace UE::Sequencer
{

struct FCachedKeys
{
	/** Handles to each of the keys */
	TArray<FKeyHandle> KeyHandles;
	/** Frame times for each of the keys in a data-dependent time-space */
	TArray<FFrameTime> KeyTimes;

	struct FCachedKey
	{
		FFrameTime Time;
		FKeyHandle Handle = FKeyHandle::Invalid();

		bool IsValid() const;
		void Reset();
	};

	void GetKeysInRange(const TRange<FFrameTime>& Range, TArrayView<const FFrameTime>* OutKeyTimes, TArrayView<const FKeyHandle>* OutHandles) const;
	void GetKeysInRangeWithBounds(const TRange<FFrameTime>& Range, TArrayView<const FFrameTime>* OutKeyTimes, TArrayView<const FKeyHandle>* OutHandles, FCachedKey* OutLeadingKey, FCachedKey* OutTrailingKey) const;
};

class IKeyExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IKeyExtension)


	virtual ~IKeyExtension()= default;

	virtual bool UpdateCachedKeys(TSharedPtr<FCachedKeys>& OutCachedKeys) const = 0;

	virtual bool GetFixedExtents(double& OutFixedMin, double& OutFixedMax) const = 0;

	virtual void DrawKeys(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<FKeyDrawParams> OutKeyDrawParams) = 0;

	virtual TUniquePtr<FCurveModel> CreateCurveModel() = 0;
};



} // namespace UE::Sequencer

#undef UE_API
