// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class ILayerBarExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ILayerBarExtension)

	virtual ~ILayerBarExtension()= default;

	virtual TRange<FFrameNumber> GetLayerBarRange() const = 0;
	virtual void OffsetLayerBar(FFrameNumber Amount) = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
