// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/Extensions/HierarchicalCacheExtension.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

enum class ELockableLockState
{
	None,
	Locked,
	PartiallyLocked,
};

/**
 * An extension for models that can be locked
 */
class ILockableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ILockableExtension)

	virtual ~ILockableExtension()= default;

	/** Returns whether this item is locked */
	virtual ELockableLockState GetLockState() const = 0;

	/** Set whether this item is locked */
	virtual void SetIsLocked(bool bIsLocked) = 0;
};

enum class ECachedLockState
{
	None                     = 0,

	Lockable                 = 1 << 0,
	LockableChildren         = 1 << 1,
	Locked                   = 1 << 2,
	PartiallyLockedChildren  = 1 << 3,
	ImplicitlyLockedByParent = 1 << 4,

	InheritedFromChildren = LockableChildren | PartiallyLockedChildren,
};
ENUM_CLASS_FLAGS(ECachedLockState)

class FLockStateCacheExtension
	: public TFlagStateCacheExtension<ECachedLockState>
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FLockStateCacheExtension);

private:

	UE_API ECachedLockState ComputeFlagsForModel(const FViewModelPtr& ViewModel) override;
	UE_API void PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedLockState& OutThisModelFlags, ECachedLockState& OutPropagateToParentFlags) override;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
