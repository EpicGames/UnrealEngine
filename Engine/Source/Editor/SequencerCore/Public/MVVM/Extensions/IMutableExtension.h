// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/Extensions/HierarchicalCacheExtension.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

/**
 * An extension for outliner nodes that can be muted
 */
class IMutableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IMutableExtension)

	virtual ~IMutableExtension()= default;

	/** Returns whether this item is muted */
	virtual bool IsMuted() const = 0;

	/** Set this item's mute state */
	virtual void SetIsMuted(bool bIsMuted) = 0;

	/** Returns whether this mutable can be muted by a parent, and should report its mute state to a parent */
	virtual bool IsInheritable() const
	{
		return true;
	}
};

enum class ECachedMuteState
{
	None                     = 0,

	Mutable                  = 1 << 0,
	MutableChildren          = 1 << 1,
	Muted                    = 1 << 2,
	PartiallyMutedChildren   = 1 << 3,
	ImplicitlyMutedByParent  = 1 << 4,
	Inheritable              = 1 << 5,

	InheritedFromChildren = MutableChildren | PartiallyMutedChildren,
};
ENUM_CLASS_FLAGS(ECachedMuteState)
SEQUENCERCORE_API ECachedMuteState CombinePropagatedChildFlags(ECachedMuteState ParentFlags, ECachedMuteState CombinedChildFlags);

class FMuteStateCacheExtension
	: public TFlagStateCacheExtension<ECachedMuteState>
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FMuteStateCacheExtension);

private:

	UE_API ECachedMuteState ComputeFlagsForModel(const FViewModelPtr& ViewModel) override;
	UE_API void PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedMuteState& OutThisModelFlags, ECachedMuteState& OutPropagateToParentFlags) override;
};


} // namespace Sequencer
} // namespace UE

#undef UE_API
