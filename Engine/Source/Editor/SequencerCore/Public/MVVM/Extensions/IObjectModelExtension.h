// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModelTypeID.h"
#include "UObject/WeakObjectPtrFwd.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

/**
 * An extension that is used for view models that represent UObject data
 */
class IObjectModelExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IObjectModelExtension)

	virtual ~IObjectModelExtension()= default;

	virtual void InitializeObject(TWeakObjectPtr<> InWeakObject) = 0;
	virtual UObject* GetObject() const = 0;
};

} // namespace UE::Sequencer

#undef UE_API
