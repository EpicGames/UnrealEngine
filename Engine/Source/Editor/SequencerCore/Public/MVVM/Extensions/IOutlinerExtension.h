// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Views/TreeViewTraits.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define UE_API SEQUENCERCORE_API

class FDragDropEvent;
class FText;
class ISequencer;
class SWidget;
struct FSlateBrush;
struct FSlateColor;
struct FSlateFontInfo;

namespace UE
{
namespace Sequencer
{

class FEditorViewModel;
class FOutlinerViewModel;
class FViewModel;
class IOutlinerExtension;

enum class EOutlinerSelectionState
{
	None                      = 0,
	SelectedDirectly          = 1 << 0,
	HasSelectedKeys           = 1 << 1,
	HasSelectedTrackAreaItems = 1 << 2,

	DescendentHasSelectedKeys = 1 << 3,
	DescendentHasSelectedTrackAreaItems = 1 << 4,
};
ENUM_CLASS_FLAGS(EOutlinerSelectionState);

enum class EOutlinerSizingFlags
{
	None                      = 0,
	DynamicSizing             = 1 << 0,
	IncludeSeparator          = 1 << 1,
	CustomHeight              = 1 << 2,
};
ENUM_CLASS_FLAGS(EOutlinerSizingFlags);

struct FOutlinerSizing
{
	FOutlinerSizing() = default;

	FOutlinerSizing(float InHeight)
		: Height(InHeight)
	{}

	FOutlinerSizing(float InHeight, float UniformPadding)
		: Height(InHeight)
		, PaddingTop(UniformPadding)
		, PaddingBottom(UniformPadding)
	{}

	friend bool operator==(const FOutlinerSizing& A, const FOutlinerSizing& B)
	{
		return 
			A.Height == B.Height && 
			A.PaddingTop == B.PaddingTop && 
			A.PaddingBottom == B.PaddingBottom && 
			A.Flags == B.Flags;
	}
	friend bool operator!=(const FOutlinerSizing& A, const FOutlinerSizing& B)
	{
		return !(A == B);
	}

	float GetSeparatorHeight() const
	{
		return EnumHasAnyFlags(Flags, EOutlinerSizingFlags::IncludeSeparator)
			? 1.f
			: 0.f;
	}

	float GetTotalHeight() const
	{
		return Height + PaddingTop + PaddingBottom + GetSeparatorHeight();
	}

	void Accumulate(const FOutlinerSizing& Other)
	{
		const float ThisSeparatorHeight  = GetSeparatorHeight();
		const float OtherSeparatorHeight = Other.GetSeparatorHeight();

		Height        = FMath::Max(Height + ThisSeparatorHeight, Other.Height + OtherSeparatorHeight);
		PaddingTop    = FMath::Max(PaddingTop, Other.PaddingTop);
		PaddingBottom = FMath::Max(PaddingBottom, Other.PaddingBottom);
		Flags |= Other.Flags;

		Height -= GetSeparatorHeight();
	}

	float Height = 0.f;
	float PaddingTop = 0.f;
	float PaddingBottom = 0.f;
	EOutlinerSizingFlags Flags = EOutlinerSizingFlags::IncludeSeparator;
};

/** Facade class to allow SOutlinerViewRow to be passed around publicly */
class ISequencerTreeViewRow : public STableRow<TWeakViewModelPtr<IOutlinerExtension>>
{
public:
	virtual bool IsColumnVisible(const FName& InColumnName) const = 0;
};

/** Parameters for creating an outliner item widget. */
struct FCreateOutlinerViewParams
{
	FCreateOutlinerViewParams(const TSharedRef<ISequencerTreeViewRow>& InTreeViewRow, const TSharedPtr<FEditorViewModel>& InEditor)
		: TreeViewRow(InTreeViewRow), Editor(InEditor)
	{}

	const TSharedRef<ISequencerTreeViewRow> TreeViewRow;
	const TSharedPtr<FEditorViewModel> Editor;
};

/** Parameters for building a context menu widget */
struct FCreateOutlinerContextMenuWidgetParams
{
	FCreateOutlinerContextMenuWidgetParams(const TSharedPtr<FEditorViewModel> InEditor)
		: Editor(InEditor)
	{}

	const TSharedPtr<FEditorViewModel> Editor;
};

/**
 * Extension interface for models that can be displayed in the outliner view.
 */
class IOutlinerExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IOutlinerExtension)

	virtual ~IOutlinerExtension()= default;

	/** Return the identifier of the owning model */
	virtual FName GetIdentifier() const = 0;

	/** Return the desired size for the owning model */
	virtual FOutlinerSizing GetOutlinerSizing() const = 0;

	/** Gets whether this outliner item is expanded */
	virtual bool IsExpanded() const = 0;

	/** Sets the expansion state of this outliner item */
	virtual void SetExpansion(bool bIsExpanded) = 0;

	/** Gets whether this outliner item has been filtered out */
	virtual bool IsFilteredOut() const = 0;

	/** Sets whether this outliner item is being filtered out */
	virtual void SetFilteredOut(bool bIsFilteredOut) = 0;

	virtual bool ShouldAnchorToTop() const = 0;

	/** Gets the outliner item's selection state */
	virtual EOutlinerSelectionState GetSelectionState() const = 0;

	/** Sets the outliner item's selection state */
	virtual void SetSelectionState(EOutlinerSelectionState InState) = 0;

	/** Toggles a bit in the outliner item's selection state */
	UE_API virtual void ToggleSelectionState(EOutlinerSelectionState InState, bool bInValue);

	/** (deprecated) Create the outliner view for the label column */
	UE_API virtual TSharedRef<SWidget> CreateOutlinerView(const FCreateOutlinerViewParams& InParams);

	/** Create the visual outliner widget for the specified column */
	UE_API virtual TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName);

	/** Whether this outliner item supports the given column. Default is true for all columns.
	 *  Override to suppress specific columns (e.g. mute/solo on transition sections). */
	virtual bool SupportsOutlinerColumn(const FName& InColumnName) const { return true; }

	/** Build the context menu widget for this outliner item */
	virtual TSharedPtr<SWidget> CreateContextMenuWidget(const FCreateOutlinerContextMenuWidgetParams& InParams) = 0;

	/** Gets the label for the outliner item */
	UE_API virtual FText GetLabel() const;

	/** Gets the font for the outliner item */
	UE_API virtual FSlateFontInfo GetLabelFont() const;

	/** Gets the color for the outliner item label */
	UE_API virtual FSlateColor GetLabelColor() const;

	/** Gets the tooltip text for the outliner item */
	UE_API virtual FText GetLabelToolTipText() const;

	/** Gets the icon for the outliner item */
	UE_API virtual const FSlateBrush* GetIconBrush() const;

	/** Gets the icon tint color for the outliner item */
	UE_API virtual FSlateColor GetIconTint() const;

	/** Gets the overlay icon for the outliner item */
	UE_API virtual const FSlateBrush* GetIconOverlayBrush() const;

	/** Gets the tooltip text for the outliner item icon */
	UE_API virtual FText GetIconToolTipText() const;

	/** Gets whether the outliner item has a background */
	UE_API virtual bool HasBackground() const;

	/** Utility method to get the full path name of the given model */
	static UE_API FString GetPathName(const FViewModel& Item);
	/** Utility method to get the full path name of the given model */
	static UE_API FString GetPathName(const TSharedPtr<FViewModel> Item);
	/** Utility method to get the full path name of the given model */
	static UE_API FString GetPathName(const TSharedPtr<const FViewModel> Item);
	/** Utility method to get the full path name of the given model */
	static UE_API FString GetPathName(const TWeakPtr<const FViewModel> Item);
	/** Utility method to get the full path name of the given model */
	static UE_API void GetPathName(const FViewModel& Item, FStringBuilderBase& OutString);
};

/**
 * Default implementation for some of the outliner extension API.
 */
class FOutlinerExtensionShim : public IOutlinerExtension
{
public:

	virtual bool IsExpanded() const override { return bIsExpanded; }
	virtual void SetExpansion(bool bInIsExpanded) override { bIsExpanded = bInIsExpanded; }
	virtual bool IsFilteredOut() const override { return bIsFilteredOut; }
	virtual void SetFilteredOut(bool bInIsFilteredOut) override { bIsFilteredOut = bInIsFilteredOut; }
	virtual EOutlinerSelectionState GetSelectionState() const override { return SelectionState; }
	virtual void SetSelectionState(EOutlinerSelectionState InState) override { SelectionState = InState; }
	virtual bool ShouldAnchorToTop() const override { return false; }

protected:

	bool bIsExpanded = false;
	bool bIsFilteredOut = false;
	EOutlinerSelectionState SelectionState = EOutlinerSelectionState::None;
};

/**
 * Extension for outliner items that accept drag and drop.
 */
class IOutlinerDropTargetOutlinerExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IOutlinerDropTargetOutlinerExtension)

	virtual ~IOutlinerDropTargetOutlinerExtension()= default;

	virtual TOptional<EItemDropZone> CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) = 0;
	virtual void PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) = 0;
};

/**
 * Extension for outliner models that want to compute their and their children's sizing.
 */
class ICompoundOutlinerExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ICompoundOutlinerExtension)

	virtual ~ICompoundOutlinerExtension()= default;

	virtual FOutlinerSizing RecomputeSizing() = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
