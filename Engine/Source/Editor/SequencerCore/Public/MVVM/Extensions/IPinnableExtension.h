// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class FViewModel;

/**
 * An extension for outliner nodes that can be pinned at the top
 */
class IPinnableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IPinnableExtension)

	virtual ~IPinnableExtension()= default;

	/** Returns whether this item is pinned, or is in a pinned branch */
	virtual bool IsPinned() const = 0;
	/** Sets whether this item is pinned */
	virtual void SetPinned(bool bInIsPinned) = 0;
	/** Lets parent items report to their children that they are in a pinned branch */
	virtual void ReportPinnedParent(bool bInIsInPinnedBranch) = 0;
};

/**
 * Default implementation for IPinnableExtension, with helper method to cache pinned
 * states for children of a pinned branch
 */
class FPinnableExtensionShim : public IPinnableExtension
{
public:

	UE_API virtual bool IsPinned() const override;
	UE_API virtual void SetPinned(bool bInIsPinned) override;
	UE_API virtual void ReportPinnedParent(bool bInIsInPinnedBranch) override;

public:
	
	/**
	 * Looks for any direct child of RootModel that is pinnable, and reports all their descendants
	 * that they are in a pinned branch, to make that lookup faster
	 */
	static UE_API void UpdateCachedPinnedState(TSharedPtr<FViewModel> RootModel);

private:

	bool bIsPinned = false;
	bool bIsInPinnedBranch = false;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
