// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "SequencerCoreFwd.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class IRecyclableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IRecyclableExtension)

	static UE_API void CallOnRecycle(const TViewModelPtr<IRecyclableExtension>& RecyclableItem);

	virtual ~IRecyclableExtension()= default;

	virtual void OnRecycle() = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
