// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "Delegates/Delegate.h"

#define UE_API SEQUENCERCORE_API

class FText;

namespace UE
{
namespace Sequencer
{

class IRenameableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IRenameableExtension)

	virtual ~IRenameableExtension()= default;

	FSimpleMulticastDelegate& OnRenameRequested() { return RenameRequestedEvent; }

	virtual bool CanRename() const = 0;
	virtual void Rename(const FText& NewName) = 0;

	UE_API bool IsRenameValid(const FText& NewName, FText& OutErrorMessage) const;

private:

	virtual bool IsRenameValidImpl(const FText& NewName, FText& OutErrorMessage) const { return true; }

private:

	FSimpleMulticastDelegate RenameRequestedEvent;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
