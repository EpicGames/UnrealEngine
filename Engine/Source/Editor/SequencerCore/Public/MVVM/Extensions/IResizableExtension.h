// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class IResizableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IResizableExtension)

	virtual ~IResizableExtension()= default;

	virtual bool IsResizable() const = 0;
	virtual void Resize(float NewSize) = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
