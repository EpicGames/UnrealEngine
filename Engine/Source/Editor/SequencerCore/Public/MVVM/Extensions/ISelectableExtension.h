// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;

namespace UE
{
namespace Sequencer
{

class FViewModel;


/**
 * Enumeration specifying the type of interaction that instigated the selection
 * This allows models to opt-into selection in some contexts while refusing
 * selection in others
 */
enum class ESelectionIntent
{
	/** Symbolic value signifying that a model is never selectable */
	Never = 0x0,

	/** Used when the selection will be persistent and visible on the UI */
	PersistentSelection = 0x1,
	/** Used exclisively for a transient context-menu selection */
	ContextMenu = 0x2,

	/** Combination that defines any intent is supported */
	Any = PersistentSelection | ContextMenu,
};
ENUM_CLASS_FLAGS(ESelectionIntent);

class ISelectableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ISelectableExtension)

	virtual ~ISelectableExtension()= default;

	virtual ESelectionIntent IsSelectable() const { return ESelectionIntent::Any; }
};


} // namespace Sequencer
} // namespace UE

#undef UE_API
