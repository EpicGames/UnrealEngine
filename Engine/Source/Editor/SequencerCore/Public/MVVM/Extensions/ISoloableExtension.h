// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/Extensions/HierarchicalCacheExtension.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

/**
 * An extension for outliner nodes that can be made 'solo'
 */
class ISoloableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ISoloableExtension)

	virtual ~ISoloableExtension()= default;

	/** Returns whether this item is solo */
	virtual bool IsSolo() const = 0;

	/** Enable or disable solo for this object */
	virtual void SetIsSoloed(bool bIsSoloed) = 0;
};

enum class ECachedSoloState
{
	None                     = 0,

	Soloable                 = 1 << 0,
	SoloableChildren         = 1 << 1,
	Soloed                   = 1 << 2,
	PartiallySoloedChildren  = 1 << 3,
	ImplicitlySoloedByParent = 1 << 4,

	InheritedFromChildren = SoloableChildren | PartiallySoloedChildren,
};
ENUM_CLASS_FLAGS(ECachedSoloState)

class FSoloStateCacheExtension
	: public TFlagStateCacheExtension<ECachedSoloState>
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FSoloStateCacheExtension)

	using Implements = TImplements<IDynamicExtension, IHierarchicalCache>;

	virtual ~FSoloStateCacheExtension() = default;

private:

	UE_API ECachedSoloState ComputeFlagsForModel(const FViewModelPtr& ViewModel) override;
	UE_API void PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedSoloState& OutThisModelFlags, ECachedSoloState& OutPropagateToParentFlags) override;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
