// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class FViewModel;
struct FViewModelChildren;
struct FViewModelListHead;

/**
 * Structure for sorting models of various types together.
 *
 * Lower values have more precedence than higher values, i.e. items associated with lower
 * values are ordered before items with higher values.
 */
struct FSortingKey
{
	/** Priority order */
	int8 Priority = 0;
	/** Identifier */
	FText DisplayName;
	/** Custom order, only relevant if >= 0 */
	int32 CustomOrder = -1;

	FSortingKey()
	{}

	FSortingKey& PrioritizeBy(int8 InOffset)
	{
		Priority -= InOffset;
		return *this;
	}

	FSortingKey& DeprioritizeBy(int8 InOffset)
	{
		Priority += InOffset;
		return *this;
	}

	/** Compare priorities, identifiers, and custom orders */
	static UE_API bool ComparePriorityFirst(const FSortingKey& A, const FSortingKey& B);
	/** Compare custom orders (if both set), priorities, and identifiers */
	static UE_API bool CompareCustomOrderFirst(const FSortingKey& A, const FSortingKey& B);

	friend bool operator==(const FSortingKey& A, const FSortingKey& B)
	{
		return A.Priority == B.Priority && 
			A.DisplayName.CompareToCaseIgnored(B.DisplayName) == 0 &&
			A.CustomOrder == B.CustomOrder;
	}
};

enum class ESortingMode
{
	PriorityFirst,
	CustomOrderFirst,
	Default = CustomOrderFirst
};

/**
 * Extension for models that can sort their children, and be sorted themselves among their siblings
 */
class ISortableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ISortableExtension)

	virtual ~ISortableExtension()= default;

	/** Called when the model should sort its immediate children */
	virtual void SortChildren() = 0;
	/** Called by the part to sort this model among its siblings */
	virtual FSortingKey GetSortingKey() const = 0;
	/** Set a custom sort index for this view model */
	virtual void SetCustomOrder(int32 InCustomOrder) = 0;

	/** Utility function for sorting children based on their sorting key */
	static UE_API void SortChildren(TSharedPtr<FViewModel> ParentModel, ESortingMode SortingMode);
	/** Utility function for sorting children based on their sorting key */
	static UE_API void SortChildren(FViewModelChildren& Children, ESortingMode SortingMode);
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
