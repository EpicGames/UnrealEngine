// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Misc/FrameTime.h"
#include "Math/Vector2D.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;
struct FPointerEvent;

namespace UE
{
namespace Sequencer
{

class IStretchOperation;
class FViewModel;
class FVirtualTrackArea;

enum class EStretchConstraint
{
	AnchorToStart, AnchorToEnd,
};

enum class EStretchResult
{
	Success, Failure
};

struct FStretchParameters
{
	double Anchor = 0.0;
	double Handle = 0.0;

	bool IsValid() const
	{
		return Anchor != Handle;
	}
};

struct FStretchScreenParameters
{
	FFrameTime DragStartPosition;
	FFrameTime CurrentDragPosition;

	FVector2D LocalMousePos;

	const FPointerEvent* MouseEvent = nullptr;
	const FVirtualTrackArea* VirtualTrackArea = nullptr;
};

class IStretchableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IStretchableExtension)

	virtual ~IStretchableExtension()= default;

	virtual void OnInitiateStretch(IStretchOperation& StretchOperation, EStretchConstraint Constraint, FStretchParameters* InOutGlobalParameters) {}

	virtual EStretchResult OnBeginStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters) { return EStretchResult::Failure; }
	virtual void OnStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters) {}
	virtual void OnEndStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParametersm, FStretchParameters* InOutParameters) {}
};

class IStretchOperation
{
public:
	virtual ~IStretchOperation() = default;

	virtual void DoNotSnapTo(TSharedPtr<FViewModel> Model) = 0;
	virtual bool InitiateStretch(TSharedPtr<FViewModel> Controller, TSharedPtr<IStretchableExtension> Target, int32 Priority, const FStretchParameters& InParams) = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
