// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class FViewModel;

enum class ETrackAreaLaneType : uint8
{
	Inline,
	Nested,
	None
};

struct FTrackAreaParameters
{
	ETrackAreaLaneType LaneType = ETrackAreaLaneType::None;
	struct
	{
		float Top = 0.f;
		float Bottom = 0.f;
	} TrackLanePadding;
};

/**
 * This extension can be added to any model in order to provide a list of models that are relevent to the track area
 */
class ITrackAreaExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ITrackAreaExtension)

	virtual ~ITrackAreaExtension()= default;

	virtual FTrackAreaParameters GetTrackAreaParameters() const = 0;
	virtual FViewModelVariantIterator GetTrackAreaModelList() const = 0;
	virtual FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const { return FViewModelVariantIterator(); }

	template<typename T>
	TTypedIterator<T, FViewModelVariantIterator> GetTrackAreaModelListAs() const
	{
		return TTypedIterator<T, FViewModelVariantIterator>(this->GetTrackAreaModelList());
	}
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
