// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;

namespace UE::Sequencer
{
/** This extension allows you to paint custom geometry in the track area. */
class ITrackAreaOverlayExtension
{
public:
	
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ITrackAreaOverlayExtension)

	/**
	 * Draws the geometry.
	 * @return The highest layer that was painted.
	 */
	virtual int32 OnPaintTrackAreaOverlay(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) = 0;

	virtual ~ITrackAreaOverlayExtension() = default;
};
}

#undef UE_API
