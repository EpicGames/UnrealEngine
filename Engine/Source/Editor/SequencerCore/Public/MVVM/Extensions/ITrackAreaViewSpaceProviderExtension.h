// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

class FTrackAreaViewModel;

/**
 * An extension that supplies view spaces to a track area.
 * Must exist as an immediate child of the root model to be effective.
 */
class ITrackAreaViewSpaceProviderExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ITrackAreaViewSpaceProviderExtension)

	virtual ~ITrackAreaViewSpaceProviderExtension()= default;

	virtual void UpdateViewSpaces(FTrackAreaViewModel& TrackAreaViewModel) = 0;
};

} // namespace UE::Sequencer

#undef UE_API
