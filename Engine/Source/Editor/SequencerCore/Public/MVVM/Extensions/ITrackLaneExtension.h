// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MVVM/ViewModelTypeID.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

#define UE_API SEQUENCERCORE_API

class FArrangedWidget;
class ISequencer;
class SWidget;
namespace UE::Sequencer { class FEditorViewModel; }
struct FGeometry;
struct FTimeToPixel;

namespace UE
{
namespace Sequencer
{

class FViewModel;
class STrackLane;
class STrackAreaView;
class FEditorViewModel;
struct FTrackLaneScreenAlignment;
struct INonLinearTimeTransform;

struct FTrackLaneVerticalArrangement
{
	float Offset = 0;
	float Height = 0;
};

struct FTrackLaneVerticalAlignment
{
	enum class ESizeMode : uint8
	{
		Proportional, Fixed
	};

	float              VSizeParam = 1.f;
	float              VPadding   = 0.f;
	EVerticalAlignment VAlign     = VAlign_Center;
	ESizeMode          Mode       = ESizeMode::Proportional;

	UE_API FTrackLaneVerticalArrangement ArrangeWithin(float LayoutHeight) const;
};

struct FTrackLaneVirtualAlignment
{
	TRange<FFrameNumber> Range;
	FTrackLaneVerticalAlignment VerticalAlignment;
	FGuid ViewSpaceID;

	bool IsVisible() const
	{
		return !Range.IsEmpty();
	}

	static FTrackLaneVirtualAlignment Fixed(const TRange<FFrameNumber>& InRange, float InFixedHeight, EVerticalAlignment InVAlign = VAlign_Center, const FGuid& InViewSpaceID = FGuid())
	{
		return FTrackLaneVirtualAlignment { InRange, { InFixedHeight, 0.f, InVAlign, FTrackLaneVerticalAlignment::ESizeMode::Fixed }, InViewSpaceID };
	}
	static FTrackLaneVirtualAlignment Proportional(const TRange<FFrameNumber>& InRange, float InStretchFactor, EVerticalAlignment InVAlign = VAlign_Center, const FGuid& InViewSpaceID = FGuid())
	{
		return FTrackLaneVirtualAlignment { InRange, { InStretchFactor, 0.f, InVAlign, FTrackLaneVerticalAlignment::ESizeMode::Proportional }, InViewSpaceID };
	}

	UE_API TOptional<FFrameNumber> GetFiniteLength() const;

	UE_API FTrackLaneScreenAlignment ToScreen(const FTimeToPixel& TimeToPixel, const FGeometry& ParentGeometry) const;
};

struct FTrackLaneScreenAlignment
{
	FTrackLaneScreenAlignment()
		: LeftPosPx(0.f)
		, WidthPx(0.f)
	{
	}

	FTrackLaneScreenAlignment(float InLeftPosPx, float InWidthPx, FTrackLaneVerticalAlignment InVerticalAlignment)
		: LeftPosPx(InLeftPosPx), WidthPx(InWidthPx), VerticalAlignment(InVerticalAlignment)
	{
	}

	TSharedPtr<INonLinearTimeTransform> NonLinearTransform;
	float LeftPosPx;
	float WidthPx;
	FTrackLaneVerticalAlignment VerticalAlignment;

	bool IsVisible() const
	{
		return WidthPx > 0.f;
	}

	UE_API FArrangedWidget ArrangeWidget(TSharedRef<SWidget> InWidget, const FGeometry& ParentGeometry) const;
};

struct FArrangedVirtualEntity
{
	TRange<FFrameNumber> Range;
	float VirtualTop, VirtualBottom;
};

/** Interface used for laying out track lane widgets in screen space */
class ITrackLaneWidgetSpace
{
public:
	virtual FTimeToPixel GetScreenSpace(const FGuid& ViewSpaceID = FGuid()) const = 0;
};

/** Base interface for track-area lanes */
class ITrackLaneWidget
{
public:

	/**
	 * Retrieve this interface as a widget
	 */
	virtual TSharedRef<const SWidget> AsWidget() const = 0;

	TSharedRef<SWidget> AsWidget()
	{
		TSharedRef<const SWidget> ConstWidget = const_cast<const ITrackLaneWidget*>(this)->AsWidget();
		return ConstCastSharedRef<SWidget>(ConstWidget);
	}

	/**
	 * Arrange this widget within its parent slot
	 */
	UE_DEPRECATED(5.6, "Please use the ITrackLaneWidgetSpace overload")
	virtual FTrackLaneScreenAlignment GetAlignment(const FTimeToPixel& TimeToPixel, const FGeometry& InParentGeometry) const
	{
		return FTrackLaneScreenAlignment();
	}
	UE_API virtual FTrackLaneScreenAlignment GetAlignment(const ITrackLaneWidgetSpace& ScreenSpace, const FGeometry& InParentGeometry) const;

	/**
	 * Gets this widget's overlap priority
	 */
	virtual int32 GetOverlapPriority() const { return 0; }

	/**
	 * Receive parent geometry for this lane in Desktop space
	 */
	virtual void ReportParentGeometry(const FGeometry& InDesktopSpaceParentGeometry) {}

	/**
	 * Whether this track lane accepts child widgets
	 */
	virtual bool AcceptsChildren() const { return false; }

	/**
	 * Add a new child to this lane
	 */
	virtual void AddChildView(TSharedPtr<ITrackLaneWidget> ChildWidget, TWeakPtr<STrackLane> InWeakOwningLane) {}
};

/** Parameters for creating a track lane widget */
struct FCreateTrackLaneViewParams
{
	FCreateTrackLaneViewParams(const TSharedPtr<FEditorViewModel> InEditor)
		: Editor(InEditor)
	{}

	const TSharedPtr<FEditorViewModel> Editor;

	TSharedPtr<FViewModel> ParentModel;

	TSharedPtr<STrackLane> OwningTrackLane;

	TSharedPtr<FTimeToPixel> TimeToPixel;
};

/** Extension for view-models that can create track lanes in the track area */
class ITrackLaneExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ITrackLaneExtension)

	virtual ~ITrackLaneExtension()= default;

	virtual TSharedPtr<ITrackLaneWidget> CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams) = 0;
	virtual FTrackLaneVirtualAlignment ArrangeVirtualTrackLaneView() const = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
