// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"

#define UE_API SEQUENCERCORE_API

struct FGuid;

namespace UE::Sequencer
{

/**
 * An extension that is used for view models that represent UObject data
 */
class IViewSpaceClientExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IViewSpaceClientExtension)

	virtual ~IViewSpaceClientExtension()= default;

	virtual FGuid GetViewSpaceID() const = 0;
	virtual void SetViewSpaceID(const FGuid&) = 0;
};


/**
 * An extension that is used for view models that represent UObject data
 */
class FViewSpaceClientExtensionShim
	: public IDynamicExtension
	, public IViewSpaceClientExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FViewSpaceClientExtensionShim)

	using Implements = TImplements<IDynamicExtension, IViewSpaceClientExtension>;

	FViewSpaceClientExtensionShim(const FGuid& InViewSpaceID)
		: ViewSpaceID(InViewSpaceID)
	{
	}

	virtual FGuid GetViewSpaceID() const override
	{
		return ViewSpaceID;
	}

	virtual void SetViewSpaceID(const FGuid& InViewSpaceID) override
	{
		ViewSpaceID = InViewSpaceID;
	}

protected:

	FGuid ViewSpaceID;
};



} // namespace UE::Sequencer

#undef UE_API
