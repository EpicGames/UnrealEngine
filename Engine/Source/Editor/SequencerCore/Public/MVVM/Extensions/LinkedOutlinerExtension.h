// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{

class FViewModel;

/**
 * Extension for a model that is linked somehow to an outliner model.
 * This is the case for example in Sequencer where the channels of the sections in the track area
 * are linked to the channel group items in the outliner.
 */
class FLinkedOutlinerExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FLinkedOutlinerExtension);

	UE_API FLinkedOutlinerExtension();

	UE_API TViewModelPtr<IOutlinerExtension> GetLinkedOutlinerItem() const;

	UE_API void SetLinkedOutlinerItem(const TViewModelPtr<IOutlinerExtension>& InOutlinerItem);

private:

	TWeakViewModelPtr<IOutlinerExtension> WeakModel;
};

/**
 * Simple mixin class for having computed sizing information on a linked outliner item.
 * This is useful for instance when the outliner item is going to compute sizing info for
 * all of its linked items.
 */
class FLinkedOutlinerComputedSizingShim
{
public:

	FOutlinerSizing GetComputedSizing() const { return ComputedSizing; }
	void SetComputedSizing(const FOutlinerSizing& InSizing) { ComputedSizing = InSizing; }

protected:

	FOutlinerSizing ComputedSizing;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
