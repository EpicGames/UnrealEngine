// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API SEQUENCERCORE_API

template<typename, typename> class TArrayView;

namespace UE
{
namespace Sequencer
{

class FViewModel;
template<typename T>
struct TViewModelExtensionCollection;

/**
 * Utility class for maintaining a list of view-models that implement a given extension
 * in a given hierarchy. The list is kept up to date when the hierarchy changes.
 */
struct FViewModelExtensionCollection : FNoncopyable
{
private:

	UE_API FViewModelExtensionCollection(FViewModelTypeID InExtensionType);
	UE_API explicit FViewModelExtensionCollection(FViewModelTypeID InExtensionType, TWeakPtr<FViewModel> InWeakModel, int32 InDesiredRecursionDepth = -1);

	UE_API virtual ~FViewModelExtensionCollection();

protected:

	virtual void OnExtensionsDirtied() {}

	UE_API void Initialize();
	UE_API void Reinitialize(TWeakPtr<FViewModel> InWeakModel, int32 InDesiredRecursionDepth = -1);
	UE_API void Destroy();

	TSharedPtr<FViewModel> GetObservedModel() const
	{
		return WeakModel.Pin();
	}

private:

	UE_API void Update() const;

	UE_API void OnHierarchyUpdated();

	UE_API void ConditionalUpdate() const;

	UE_API void DestroyImpl();

	template<typename T>
	TArrayView<T* const> GetExtensions() const
	{
		// Conditionally update any ptrs before we allow any access to them
		ConditionalUpdate();

		void* const * BaseExtensions = ExtensionContainer.GetData();
		T* const * Data = reinterpret_cast<T* const *>(BaseExtensions);
		return MakeArrayView(Data, ExtensionContainer.Num());
	}

private:
	template<typename T>
	friend struct TViewModelExtensionCollection;

	mutable TArray<void*> ExtensionContainer;
	TWeakPtr<FViewModel> WeakModel;
	FDelegateHandle OnHierarchyUpdatedHandle;
	FViewModelTypeID ExtensionType;
	int32 DesiredRecursionDepth;
	mutable bool bNeedsUpdate;
};

/**
 * Strongly typed version of FViewModelExtensionCollection
 */
template<typename T>
struct TViewModelExtensionCollection : FViewModelExtensionCollection
{
	TViewModelExtensionCollection()
		: FViewModelExtensionCollection(T::ID)
	{}

	explicit TViewModelExtensionCollection(TWeakPtr<FViewModel> InWeakModel)
		: FViewModelExtensionCollection(T::ID, InWeakModel)
	{}

	explicit TViewModelExtensionCollection(TWeakPtr<FViewModel> InWeakModel, int32 InDesiredRecursionDepth)
		: FViewModelExtensionCollection(T::ID, InWeakModel, InDesiredRecursionDepth)
	{}

	TArrayView<T* const> GetExtensions() const
	{
		return FViewModelExtensionCollection::GetExtensions<T>();
	}

	template<typename Predicate>
	void FilterExtensions(Predicate&& InPredicate)
	{
		for (int32 Index = ExtensionContainer.Num()-1; Index >= 0; --Index)
		{
			if (!InPredicate(static_cast<T*>(ExtensionContainer[Index])))
			{
				ExtensionContainer.RemoveAt(Index, EAllowShrinking::No);
			}
		}
	}
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
