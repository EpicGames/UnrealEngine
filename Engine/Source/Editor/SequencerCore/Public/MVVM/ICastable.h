// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MVVM/ViewModelTypeID.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Invoke.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "MVVM/CastableTypeTable.h"

#include <type_traits>

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

struct FCastableTypeTable;
class FDynamicExtensionContainer;

class ICastable
{
public:

	template<typename RetType, typename InterfaceType, typename ...ArgTypes, typename ...ParamTypes>
	RetType CastedCall(RetType(InterfaceType::*MemberFunc)(ArgTypes...), ParamTypes&&... InArgs)
	{
		return Invoke(MemberFunc, this->CastThisChecked<InterfaceType>(), Forward<ParamTypes>(InArgs)...);
	}

	template<typename RetType, typename InterfaceType, typename ...ArgTypes, typename ...ParamTypes>
	RetType CastedCall(RetType(InterfaceType::*MemberFunc)(ArgTypes...), ParamTypes&&... InArgs) const
	{
		return Invoke(MemberFunc, this->CastThisChecked<InterfaceType>(), Forward<ParamTypes>(InArgs)...);
	}

	template<typename T>
	static T* CastWeakPtr(const TWeakPtr<ICastable>& InWeakPtr)
	{
		if (TSharedPtr<ICastable> Pinned = InWeakPtr.Pin())
		{
			return Pinned->CastThis<T>();
		}
		return nullptr;
	}

	template<typename T>
	static TSharedPtr<T> CastWeakPtrShared(const TWeakPtr<ICastable>& InWeakPtr)
	{
		if (TSharedPtr<ICastable> Pinned = InWeakPtr.Pin())
		{
			return TSharedPtr<T>(Pinned, Pinned->CastThis<T>());
		}
		return nullptr;
	}

	bool IsA(FViewModelTypeID InType) const
	{
		return CastRaw(InType) != nullptr;
	}

	template<typename T>
	bool IsA() const
	{
		return CastThis<T>() != nullptr;
	}

	UE_API void* CastRaw(FViewModelTypeID InType);
	UE_API const void* CastRaw(FViewModelTypeID InType) const;

	template<typename T>
	T* CastThis()
	{
		return static_cast<T*>(const_cast<void*>(CastRaw(T::ID)));
	}

	template<typename T>
	const T* CastThis() const
	{
		return static_cast<const T*>(CastRaw(T::ID));
	}

	template<typename T>
	T* CastThisChecked()
	{
		T* Result = CastThis<T>();
		check(Result);
		return Result;
	}

	template<typename T>
	const T* CastThisChecked() const
	{
		const T* Result = CastThis<const T>();
		check(Result);
		return Result;
	}

	const FCastableTypeTable& GetTypeTable() const
	{
		// This should never trip - it should always be initialzed on construction by a FAutoRegisterTypeTable member
		checkSlow(TypeTable);
		return *TypeTable;
	}

protected:

	/** Struct that assigns an ICastable instance's TypeTable. See UE_SEQUENCER_DECLARE_CASTABLE for usage. */
	struct FAutoRegisterTypeTable
	{
		template<typename T>
		FAutoRegisterTypeTable(T* InThis)
		{
			InThis->ID.Register();
			const_cast<const FCastableTypeTable*&>(InThis->TypeTable) = InThis->ID.GetTypeTable();
		}
	};

private:

	friend FAutoRegisterTypeTable;
	const FCastableTypeTable* const TypeTable = nullptr;

protected:
	FDynamicExtensionContainer* DynamicTypes = nullptr;
};

template<typename ...T>
struct TImplements
{
};


// MSVC has a compiler bug with zero-sized members of derived types that causes bad code generation for castable types
// So don't use UE_NO_UNIQUE_ADDRESS on MSVC
// https://developercommunity.visualstudio.com/t/Bad-code-generation-with-empty-first-mem/10900015
#ifdef _MSC_VER
	#define UE_SEQUENCER_NO_UNIQUE_ADDRESS 
#else
	#define UE_SEQUENCER_NO_UNIQUE_ADDRESS UE_NO_UNIQUE_ADDRESS
#endif

#define UE_SEQUENCER_DECLARE_CASTABLE(ThisType, ...)																																\
	UE_SEQUENCER_DECLARE_CASTABLE_API(, ThisType, __VA_ARGS__)

#define UE_SEQUENCER_DECLARE_CASTABLE_API(MODULE_API, ThisType, ...)																													\
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(MODULE_API, ThisType);																													\
	using Implements = ::UE::Sequencer::TImplements<__VA_ARGS__>;																														\
	UE_SEQUENCER_NO_UNIQUE_ADDRESS ::UE::Sequencer::ICastable::FAutoRegisterTypeTable AutoRegisterTypeTable = ::UE::Sequencer::ICastable::FAutoRegisterTypeTable(static_cast<ThisType*>(this));	\

#define UE_SEQUENCER_DEFINE_CASTABLE(ThisType)																																		\
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ThisType);


} // namespace UE::Sequencer

#undef UE_API
