// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Misc/Optional.h"
#include "MVVM/Selection/SequencerCoreSelectionTypes.h"
#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"


namespace UE::Sequencer
{

class FSelectionBase;
class FOutlinerSelection;

/**
 * Core selection class that manages multiple different selection sets through FSelectionBase.
 *
 * Implementers of this class should create a derived type that defines each of its selection sets
 * by calling AddSelectionSet in its constructor. This enables a central implementation for event
 * suppression and broadcast.
 */
class FSequencerCoreSelection : public TSharedFromThis<FSequencerCoreSelection>
{
public:

	/**
	 * Event that is broadcast when any selection state inside this class has changed
	 & after all scoped suppressors have been destroyed
	 */
	FSimpleMulticastDelegate OnChanged;

	/**
	 * Retrieve the serial number that identifies this selection's state.
	 */
	uint32 GetSerialNumber() const
	{
		return SerialNumber;
	}
	
	/**
	 * Check whether this selection is currently broadcasting its selection changed events
	 */
	bool IsTriggeringSelectionChangedEvents() const
	{
		return bTriggeringSelectionEvents;
	}

	/**
	 * Retrieve a scoped object that will suppress selection events for the duration of its lifetime on the stack
	 */
	[[nodiscard]] SEQUENCERCORE_API FSelectionEventSuppressor SuppressEvents();

	/**
	 * Retrieve a scoped object that will suppress selection events for the duration of its lifetime.
	 * This function should only be used for longer running suppression such as those as part of a mouse-down operation.
	 * The suppressor can be destroyed by resetting the unique pointer
	 */
	[[nodiscard]] SEQUENCERCORE_API TUniquePtr<FSelectionEventSuppressor> SuppressEventsLongRunning();

	/**
	 * Empty all the selection states within this selection class
	 */
	SEQUENCERCORE_API void Empty();

public:

	virtual TSharedPtr<      FOutlinerSelection> GetOutlinerSelection()       { return nullptr; }
	virtual TSharedPtr<const FOutlinerSelection> GetOutlinerSelection() const { return nullptr; }

protected:

	/**
	 * Protected constructor - to be implemented by a derived class
	 */
	FSequencerCoreSelection()
		: ScopedSuppressionCount(0)
		, SerialNumber(0)
		, bTriggeringSelectionEvents(false)
	{}

	virtual ~FSequencerCoreSelection() = default;

	/**
	 * Add a selection set to this class - selection set should be a member of the derived implementation
	 * of this class so as to ensure its lifetime matches this owner
	 */
	void AddSelectionSet(FSelectionBase* InSelection)
	{
		SelectionSets.Add(InSelection);
		InSelection->Owner = this;
	}

private:

	/**
	 * Function that is called after increasing the serial number, but before triggering any OnChanged events
	 * This allows derived classes to update any cached data they might want to generate from the selection before publicizing the change
	 */
	virtual void PreBroadcastChangeEvent() {}

	/**
	 * Function that is called the first time within a FSelectionEventSuppressor scope that a selection set is changed.
	 * Care should be taken to not cause infinite loops or non-deterministc modification or other selection sets when overriding this function .
	 */
	virtual void PreSelectionSetChangeEvent(FSelectionBase* InSelectionSet) {}

	SEQUENCERCORE_API void BroadcastSelectionChanged();

private:

	friend FSelectionBase;
	friend FSelectionEventSuppressor;

	/** Array of registered selection sets (contained in a derived type), registerd thorugh AddSelectionSet */
	TArray<FSelectionBase*> SelectionSets;
	/** The number of outstanding scoped event suppression objects. */
	int32 ScopedSuppressionCount;
	/** A monotonically increasing integer that changes whenever anything within this selection is changed */
	uint32 SerialNumber;
	/** Boolean to guard against re-entrancy when triggering selection events */
	bool bTriggeringSelectionEvents;
};

} // namespace UE::Sequencer
