// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModelPtr.h"


namespace UE::Sequencer
{

class FViewModel;
class FSequencerCoreSelection;

template<typename MixinType, typename KeyType>
class TSelectionSetBase;

template<typename StorageKeyType, typename FilterType>
struct TFilteredViewModelSelectionIterator;

/**
 * Minimal traits class for iterator implementations.
 * Can be specialized to customize selection set iteration behavior.
 */
template<typename T>
struct TSelectionSetIteratorImpl
{
	/**
	 * Access the current value of the iterator. By default this just returns the value directly.
	 */
	const T& operator*() const
	{
		return *IteratorBegin;
	}

protected:

	explicit TSelectionSetIteratorImpl(typename TSet<T>::TRangedForConstIterator InIteratorBegin, typename TSet<T>::TRangedForConstIterator InIteratorEnd)
	: IteratorBegin(InIteratorBegin), IteratorEnd(InIteratorEnd)
	{}

	/**
	 * Skip invalid iterations - by default all items are valid
	 */
	constexpr void SkipInvalid()
	{}

	typename TSet<T>::TRangedForConstIterator IteratorBegin;
	typename TSet<T>::TRangedForConstIterator IteratorEnd;
};

/**
 * Selection iterator class that iterates a given selection set using specialized behavior for weak pointers
 */
template<typename T, typename ImplType = TSelectionSetIteratorImpl<T>>
struct TSelectionSetIteratorState : ImplType
{
	explicit TSelectionSetIteratorState(typename TSet<T>::TRangedForConstIterator InIteratorBegin, typename TSet<T>::TRangedForConstIterator InIteratorEnd)
		: ImplType(InIteratorBegin, InIteratorEnd)
	{
		this->SkipInvalid();
	}

	void operator++()
	{
		++this->IteratorBegin;
		this->SkipInvalid();
	}

	friend bool operator!=(const TSelectionSetIteratorState<T, ImplType>& A, const TSelectionSetIteratorState<T, ImplType>& B)
	{
		return A.IteratorBegin != B.IteratorBegin && ensure(A.IteratorEnd == B.IteratorEnd);
	}
};


template<typename KeyType, typename FilterType>
struct TFilteredSelectionSetIteratorImpl
{
	/**
	 * Access the current value of the iterator. By default this just returns the value directly.
	 */
	TViewModelPtr<FilterType> operator*() const
	{
		return CurrentValue;
	}

protected:

	explicit TFilteredSelectionSetIteratorImpl(typename TSet<KeyType>::TRangedForConstIterator InIteratorBegin, typename TSet<KeyType>::TRangedForConstIterator InIteratorEnd)
		: IteratorBegin(InIteratorBegin), IteratorEnd(InIteratorEnd)
	{}

	void SkipInvalid()
	{
		CurrentValue = nullptr;
		while (IteratorBegin != IteratorEnd)
		{
			CurrentValue = CastViewModel<FilterType>(IteratorBegin->Pin());
			if (CurrentValue)
			{
				break;
			}
			++IteratorBegin;
		}
	}

	typename TSet<KeyType>::TRangedForConstIterator IteratorBegin;
	typename TSet<KeyType>::TRangedForConstIterator IteratorEnd;
	TViewModelPtr<FilterType> CurrentValue;
};

/**
 * Filtered iterator for a TSelectionSet<TWeakViewModelPtr<T>> that iterates only the items of a specific filtered type
 */
template<typename StorageKeyType, typename FilterType>
using TFilteredViewModelSelectionIteratorState = TSelectionSetIteratorState<StorageKeyType, TFilteredSelectionSetIteratorImpl<StorageKeyType, FilterType>>;

/**
 * Iterator for iterating unique fragment selection sets
 */
template<typename KeyType>
struct TUniqueFragmentSelectionSetIterator
{
	void operator++()
	{
		++Iterator;
	}

	operator KeyType() const
	{
		return *Iterator;
	}

	KeyType operator*() const
	{
		return *Iterator;
	}

	friend bool operator!=(const TUniqueFragmentSelectionSetIterator<KeyType>& A, const TUniqueFragmentSelectionSetIterator<KeyType>& B)
	{
		return A.Iterator != B.Iterator;
	}

private:

	template<typename, typename>
	friend class TUniqueFragmentSelectionSet;

	explicit TUniqueFragmentSelectionSetIterator(typename TSet<KeyType>::TRangedForConstIterator InIterator)
		: Iterator(InIterator)
	{}

	typename TSet<KeyType>::TRangedForConstIterator Iterator;
};

template<typename StorageKeyType, typename FilterType>
struct TFilteredViewModelSelectionIterator
{
	const TSet<StorageKeyType>* SelectionSet;

	FORCEINLINE TFilteredViewModelSelectionIteratorState<StorageKeyType, FilterType> begin() const { return TFilteredViewModelSelectionIteratorState<StorageKeyType, FilterType>(SelectionSet->begin(), SelectionSet->end()); }
	FORCEINLINE TFilteredViewModelSelectionIteratorState<StorageKeyType, FilterType> end() const   { return TFilteredViewModelSelectionIteratorState<StorageKeyType, FilterType>(SelectionSet->end(), SelectionSet->end());   }
};


/**
 * Specialization for TWeakPtr types that skips invalid pointers and exposes iterators as TSharedPtr
 */
template<typename T>
struct TSelectionSetIteratorImpl<TWeakPtr<T>>
{
	TSharedPtr<T> operator*() const
	{
		return IteratorBegin->Pin();
	}

protected:

	explicit TSelectionSetIteratorImpl(typename TSet<TWeakPtr<T>>::TRangedForConstIterator InIteratorBegin, typename TSet<TWeakPtr<T>>::TRangedForConstIterator InIteratorEnd)
		: IteratorBegin(InIteratorBegin), IteratorEnd(InIteratorEnd)
	{}

	void SkipInvalid()
	{
		while (IteratorBegin != IteratorEnd && !IteratorBegin->Pin())
		{
			++IteratorBegin;
		}
	}

	typename TSet<TWeakPtr<T>>::TRangedForConstIterator IteratorBegin;
	typename TSet<TWeakPtr<T>>::TRangedForConstIterator IteratorEnd;
};


/**
 * Specialization for TWeakViewModelPtr types that skips invalid pointers and exposes iterators as TViewModelPtr
 */
template<typename T>
struct TSelectionSetIteratorImpl<TWeakViewModelPtr<T>>
{
	TViewModelPtr<T> operator*() const
	{
		return IteratorBegin->Pin();
	}

protected:

	explicit TSelectionSetIteratorImpl(typename TSet<TWeakViewModelPtr<T>>::TRangedForConstIterator InIteratorBegin, typename TSet<TWeakViewModelPtr<T>>::TRangedForConstIterator InIteratorEnd)
		: IteratorBegin(InIteratorBegin), IteratorEnd(InIteratorEnd)
	{}

	void SkipInvalid()
	{
		while (IteratorBegin != IteratorEnd && !IteratorBegin->Pin())
		{
			++IteratorBegin;
		}
	}

	typename TSet<TWeakViewModelPtr<T>>::TRangedForConstIterator IteratorBegin;
	typename TSet<TWeakViewModelPtr<T>>::TRangedForConstIterator IteratorEnd;
};


} // namespace UE::Sequencer