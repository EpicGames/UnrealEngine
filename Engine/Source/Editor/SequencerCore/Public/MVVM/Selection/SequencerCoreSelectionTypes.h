// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"
#include "MVVM/Selection/SequencerCoreSelectionIterators.h"


namespace UE::Sequencer
{

class FSequencerCoreSelection;

/**
 * Base class for all selection sets that can be added to an FSequencerCoreSelection owner.
 */
class FSelectionBase
{
public:

	/**
	 * Broadcast if this selection set has changed when all event suppressors have been destroyed
	 */
	FSimpleMulticastDelegate OnChanged;

	/**
	 * Empty this selection set
	 */
	SEQUENCERCORE_API void Empty();

	/**
	 * Check whether this selection set has reported changes that are currently pending
	 */
	bool HasPendingChanges() const
	{
		return bSelectionChanged;
	}

protected:

	/**
	 * Should only be used by derived types
	 */
	FSelectionBase()
		: Owner(nullptr)
		, bSelectionChanged(false)
	{}

	virtual ~FSelectionBase() = default;

	/**
	 * Retrieve the owner of this instance, assuming it has been added to one
	 */
	FSequencerCoreSelection* GetOwner() const
	{
		return Owner;
	}

	/**
	 * Called by derived classes when they have changed
	 */
	SEQUENCERCORE_API void ReportChanges(bool bInSelectionChanged = true);

private:

	/**
	 * Abstract function for emptying this selection
	 */
	virtual void EmptyImpl() = 0;

	friend FSequencerCoreSelection;

	/** The owning instance of this class */
	FSequencerCoreSelection* Owner;
	/** Flag indicating whether this selection has changed within the current scope or not */
	bool bSelectionChanged;
};


/**
 * Typed selection set for keeping track of an externally provided key type
 * KeyType is what is stored within the selection set (ie, an FObjectKey, TWeakPtr or int32)
 * MixinType is the type of the derived class. This allows for compile-time injection of
 * OnSelect/OnDeselect behavior without needing virtual function calls:
 *
 *      bool OnSelectItem(const KeyType&){ return true; }
 *      void OnDeselectItem(const KeyType&){}
 *
 * Example implementation:
 * class FMySelection : TSelectionSetBase<FMySelection, uint32>
 * {
 * public:
 *      // Both these functions are optional:
 *      bool OnSelectItem(uint32 In){ return In != 0; }
 *      void OnDeselectItem(uint32){}
 *  };
 *
 * Iteration is supported for all types (weak ptrs are skipped and iterated as shared pointers):
 *
 *     class FMySelection : TSelectionSetBase<FMySelection, FWeakViewModelPtr>{};
 *     FMySelection& Selection = ...;
 *     for (FViewModelPtr ViewModel : Selection)
 *     {
 *         //...;
 *     }
 *
 *     // Filtered iteration is also supported for TWeakViewModelPtr<> types:
 *     for (TViewModelPtr<IOutlinerExtension> Outliner : Selection.Filter<IOutlinerExtension>())
 *     {
 *         // Only viewmodels that implement the IOutlinerExtension will be iterated
 *     }
 */
template<typename MixinType, typename KeyType>
class TSelectionSetBase : public FSelectionBase
{
public:

	/**
	 * Returns the number of selected items (including potentially invalid items)
	 */
	int32 Num() const
	{
		return SelectionSet.Num();
	}

	/**
	 * Returns the underlying selection set  (including potentially invalid items)
	 */
	const TSet<KeyType>& GetSelected() const
	{
		return SelectionSet;
	}

	/**
	 * Check whether a given item is currently selected
	 */
	bool IsSelected(KeyType InKey) const
	{
		return SelectionSet.Contains(InKey);
	}

	/**
	 * Select a key, optionally supplying a boolean to receieve a value indicating whether it was already selected or not
	 *
	 * @param  InKey                The item to select
	 * @param  OutNewlySelected     (Optional) When non-null, will be assigned true of false based on whether this item was newly selected as a result of this call
	 */
	void Select(KeyType InKey, bool* OutNewlySelected = nullptr)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());

		const bool bAlreadySelected = SelectionSet.Contains(InKey);
		if (!bAlreadySelected)
		{
			SelectRange(MakeArrayView(&InKey, 1), OutNewlySelected);
		}
		else if (OutNewlySelected)
		{
			*OutNewlySelected = false;
		}
	}

	/**
	 * Selects a range of keys, optionally supplying a boolean to receieve a value indicating whether any were newly selected or not
	 *
	 * @param  Range               An arbitrary range of items to select. Must be implicitly convertible to KeyType.
	 * @param  OutNewlySelected    (Optional) When non-null, will be assigned true of false based on whether this item was newly selected as a result of this call
	 */
	template<typename RangeType>
	void SelectRange(RangeType Range, bool* OutAnySelected = nullptr)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());

		const int32 PreviousNum = SelectionSet.Num();

		MixinType* This = static_cast<MixinType*>(this);

		for (const auto& Key : Range)
		{
			if (static_cast<MixinType*>(this)->OnSelectItem(Key))
			{
				SelectionSet.Add(Key);
			}
		}

		if (SelectionSet.Num() != PreviousNum)
		{
			ReportChanges();

			if (OutAnySelected)
			{
				*OutAnySelected = true;
			}
		}
		else if (OutAnySelected)
		{
			*OutAnySelected = false;
		}
	}

	/**
	 * Replace this selection set with the contents from another
	 *
	 * @param  OtherSelection      The selection set to replace this one with
	 */
	void ReplaceWith(TSelectionSetBase<MixinType, KeyType>&& OtherSelection)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());

		const int32 PreviousNum = SelectionSet.Num();

		for (const KeyType& Key : SelectionSet)
		{
			static_cast<MixinType*>(this)->OnDeselectItem(Key);
		}

		SelectionSet = MoveTemp(OtherSelection.SelectionSet);
		OtherSelection.SelectionSet.Empty();

		for (const KeyType& Key : SelectionSet)
		{
			static_cast<MixinType*>(this)->OnSelectItem(Key);
		}

		ReportChanges();
	}

	/**
	 * Deselect an item
	 */
	void Deselect(KeyType InKey)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());

		const int32 PreviousNum = SelectionSet.Num();
		static_cast<MixinType*>(this)->OnDeselectItem(InKey);

		SelectionSet.Remove(InKey);

		ReportChanges(SelectionSet.Num() != PreviousNum);
	}

	/**
	 * Empty this selection
	 */
	void Empty()
	{
		if (SelectionSet.Num() != 0)
		{
			FSelectionEventSuppressor Suppressor(GetOwner());

			for (const KeyType& Key : SelectionSet)
			{
				static_cast<MixinType*>(this)->OnDeselectItem(Key);
			}

			SelectionSet.Empty();
			ReportChanges();
		}
	}

	/**
	 * * Remove all selected items that pass the specified filter
	 */
	template<typename Filter>
	void RemoveByPredicate(Filter&& InFilter)
	{
		if (SelectionSet.Num() != 0)
		{
			FSelectionEventSuppressor Suppressor(GetOwner());

			bool bChanged = false;
			for (auto It = SelectionSet.CreateIterator(); It; ++It)
			{
				const KeyType Key = *It;
				if (InFilter(Key))
				{
					It.RemoveCurrent();
					bChanged = true;
				}
			}

			ReportChanges(bChanged);
		}
	}

public:

	/** Ranged-for iteration support. Do not use directly */
	FORCEINLINE TSelectionSetIteratorState<KeyType> begin() const { return TSelectionSetIteratorState<KeyType>(SelectionSet.begin(), SelectionSet.end()); }
	FORCEINLINE TSelectionSetIteratorState<KeyType> end() const   { return TSelectionSetIteratorState<KeyType>(SelectionSet.end(), SelectionSet.end());   }

	/** Fallback functions for use when derived classes do not implement them. Should be completely compiled out */
	FORCEINLINE static constexpr bool OnSelectItem(const KeyType&){ return true; }
	FORCEINLINE static constexpr void OnDeselectItem(const KeyType&){}

protected:

	void EmptyImpl() override
	{
		Empty();
	}

	TSet<KeyType> SelectionSet;
};



/**
 * Selection set that defines a fragment selection of a view model where each fragment
 * is globally unique. A good example is FKeyHandle or any other monotomic counter.
 * Selection must be associated with a view model type defined by the template parameter
 * ModelType.
 */
template<typename KeyType, typename ModelType>
class TUniqueFragmentSelectionSet : public FSelectionBase
{
public:

	/**
	 * Returns the number of selected items (including potentially invalid items)
	 */
	int32 Num() const
	{
		return SelectionSet.Num();
	}

	/**
	 * Returns the underlying selection set (including potentially invalid items)
	 */
	const TSet<KeyType>& GetSelected() const
	{
		return SelectionSet;
	}

	/**
	 * Retrieve the view model associated with the specified fragment selection
	 */
	TViewModelPtr<ModelType> GetModelForKey(KeyType InKey) const
	{
		return KeyToViewModel.FindRef(InKey).Pin();
	}

	/**
	 * Check whether a given item is currently selected
	 */
	bool IsSelected(KeyType InKey) const
	{
		return SelectionSet.Contains(InKey);
	}

	/**
	 * Select a key, optionally supplying a boolean to receieve a value indicating whether it was already selected or not
	 *
	 * @param  InKey                The item to select
	 * @param  OutNewlySelected     (Optional) When non-null, will be assigned true of false based on whether this item was newly selected as a result of this call
	 */
	void Select(TViewModelPtr<ModelType> InOwner, KeyType InKey, bool* OutNewlySelected = nullptr)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());
		SelectRange(InOwner, MakeArrayView(&InKey, 1), OutNewlySelected);
	}

	/**
	 * Selects a range of keys, optionally supplying a boolean to receieve a value indicating whether any were newly selected or not
	 *
	 * @param  Range               An arbitrary range of items to select. Must be implicitly convertible to KeyType.
	 * @param  OutNewlySelected    (Optional) When non-null, will be assigned true of false based on whether this item was newly selected as a result of this call
	 */
	template<typename RangeType>
	void SelectRange(TViewModelPtr<ModelType> InOwner, RangeType Range, bool* OutAnySelected = nullptr)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());

		const int32 PreviousNum = SelectionSet.Num();

		for (const auto& Key : Range)
		{
			bool bAlreadySelected = false;
			SelectionSet.Add(Key, &bAlreadySelected);
			if (!bAlreadySelected)
			{
				KeyToViewModel.Add(Key, InOwner);
			}
		}

		if (SelectionSet.Num() != PreviousNum)
		{
			ReportChanges();

			if (OutAnySelected)
			{
				*OutAnySelected = true;
			}
		}
		else if (OutAnySelected)
		{
			*OutAnySelected = false;
		}
	}

	/**
	 * Replace this selection set with the contents from another
	 *
	 * @param  OtherSelection      The selection set to replace this one with
	 */
	void ReplaceWith(TUniqueFragmentSelectionSet<KeyType, ModelType>&& OtherSelection)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());

		const int32 PreviousNum = SelectionSet.Num();

		SelectionSet = MoveTemp(OtherSelection.SelectionSet);
		KeyToViewModel = MoveTemp(OtherSelection.KeyToViewModel);

		OtherSelection.SelectionSet.Empty();
		OtherSelection.KeyToViewModel.Empty();

		ReportChanges(SelectionSet.Num() != PreviousNum);
	}

	/**
	 * Deselect an item
	 */
	void Deselect(KeyType InKey)
	{
		FSelectionEventSuppressor Suppressor(GetOwner());

		const int32 PreviousNum = SelectionSet.Num();

		SelectionSet.Remove(InKey);
		KeyToViewModel.Remove(InKey);

		ReportChanges(SelectionSet.Num() != PreviousNum);
	}

	/**
	 * Empty this selection
	 */
	void Empty()
	{
		if (SelectionSet.Num() != 0)
		{
			FSelectionEventSuppressor Suppressor(GetOwner());
			SelectionSet.Empty();
			KeyToViewModel.Empty();
			ReportChanges();
		}
	}

	/**
	 * Remove all selected items that pass the specified filter
	 */
	template<typename Filter>
	void RemoveByPredicate(Filter&& InFilter)
	{
		if (SelectionSet.Num() != 0)
		{
			FSelectionEventSuppressor Suppressor(GetOwner());

			bool bChanged = false;
			for (auto It = SelectionSet.CreateIterator(); It; ++It)
			{
				const KeyType Key = *It;
				if (InFilter(Key))
				{
					KeyToViewModel.Remove(Key);
					It.RemoveCurrent();
					bChanged = true;
				}
			}

			ReportChanges(bChanged);
		}
	}

public:

	FORCEINLINE TUniqueFragmentSelectionSetIterator<KeyType> begin() const { return TUniqueFragmentSelectionSetIterator<KeyType>(SelectionSet.begin()); }
	FORCEINLINE TUniqueFragmentSelectionSetIterator<KeyType> end() const   { return TUniqueFragmentSelectionSetIterator<KeyType>(SelectionSet.end());   }

private:

	void EmptyImpl() override
	{
		Empty();
	}

	/** The unique selection set of fragments */
	TSet<KeyType> SelectionSet;
	/** Reverse mapping from item to its owning model */
	TMap<KeyType, TWeakViewModelPtr<ModelType>> KeyToViewModel;
};

} // namespace UE::Sequencer
