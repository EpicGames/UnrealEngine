// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "SequencerCoreFwd.h"
#include "MVVM/Selection/SequencerCoreSelectionTypes.h"
//#include "MVVM/Extensions/IOutlinerExtension.h"


namespace UE::Sequencer
{

class IOutlinerExtension;

class FOutlinerSelection : public TSelectionSetBase<FOutlinerSelection, TWeakViewModelPtr<IOutlinerExtension>>
{
public:

	/**
	 * Filter this selection set based on the specified filter type
	 */
	template<typename FilterType>
	TFilteredViewModelSelectionIterator<TWeakViewModelPtr<IOutlinerExtension>, FilterType> Filter() const
	{
		return TFilteredViewModelSelectionIterator<TWeakViewModelPtr<IOutlinerExtension>, FilterType>{ &GetSelected() };
	}

private:

	friend TSelectionSetBase<FOutlinerSelection, TWeakViewModelPtr<IOutlinerExtension>>;

	SEQUENCERCORE_API bool OnSelectItem(const TWeakViewModelPtr<IOutlinerExtension>& WeakViewModel);
	SEQUENCERCORE_API void OnDeselectItem(const TWeakViewModelPtr<IOutlinerExtension>& WeakViewModel);
};


} // namespace UE::Sequencer