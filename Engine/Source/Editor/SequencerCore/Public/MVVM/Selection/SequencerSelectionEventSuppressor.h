// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Sequencer
{

class FSequencerCoreSelection;

/**
 * Scoped object type that temporarily suppresses broadcast of selection events
 * @see FSequencerCoreSelection::SuppressEvents
 */
struct FSelectionEventSuppressor
{
	SEQUENCERCORE_API FSelectionEventSuppressor(FSequencerCoreSelection* InSelection);
	SEQUENCERCORE_API ~FSelectionEventSuppressor();

	FSelectionEventSuppressor(const FSelectionEventSuppressor&) = delete;
	void operator=(const FSelectionEventSuppressor&) = delete;

	FSelectionEventSuppressor(FSelectionEventSuppressor&&) = delete;
	void operator=(FSelectionEventSuppressor&&) = delete;

private:
	FSequencerCoreSelection* Selection;
};


} // namespace UE::Sequencer