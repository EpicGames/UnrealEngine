// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE
{
namespace Sequencer
{

template<typename T>
class TSharedListItem : public TSharedFromThis<T>
{
public:
	virtual ~TSharedListItem()
	{
		Unlink();
	}

	TSharedPtr<T> Next() const
	{
		return NextLink;
	}

	void Link(TSharedPtr<T>& Head)
	{
		if (Head)
		{
			Head->PrevLink = &NextLink;
		}

		NextLink = Head;
		PrevLink = &Head;
		Head = this->AsShared();
	}

	void Unlink()
	{
		if (NextLink)
		{
			NextLink->PrevLink = PrevLink;
		}
		if (PrevLink)
		{
			*PrevLink = NextLink;
		}

		NextLink = nullptr;
		PrevLink = nullptr;
	}

private:

	TSharedPtr<T> NextLink;
	TSharedPtr<T>* PrevLink = nullptr;
};


template<typename T>
class TWeakListItem
{
public:

};

} // namespace Sequencer
} // namespace UE