// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "SequencerCoreFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#define UE_API SEQUENCERCORE_API

class FDelegateHandle;

namespace UE
{
namespace Sequencer
{

class FViewModel;

class FSharedViewModelData
	: public FViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FSharedViewModelData, FViewModel);

	UE_API void PreHierarchicalChange(const TSharedPtr<FViewModel>& InChangedModel);
	UE_API void BroadcastHierarchicalChange(const TSharedPtr<FViewModel>& InChangedModel);

	UE_API void ReportLatentHierarchicalOperations();

	UE_API FSimpleMulticastDelegate& SubscribeToHierarchyChanged(const TSharedPtr<FViewModel>& InModel);
	UE_API void UnsubscribeFromHierarchyChanged(const TSharedPtr<FViewModel>& InModel, FDelegateHandle InHandle);
	UE_API void UnsubscribeFromHierarchyChanged(const TSharedPtr<FViewModel>& InModel, FDelegateUserObjectConst InUserObject);

	UE_API void PurgeStaleHandlers();

private:
	friend FViewModelHierarchyOperation;

	TMap<TWeakPtr<FViewModel>, FSimpleMulticastDelegate> HierarchyChangedEventsByModel;

	FViewModelHierarchyOperation* CurrentHierarchicalOperation = nullptr;
	TUniquePtr<FViewModelHierarchyOperation> LatentOperation;
};


} // namespace Sequencer
} // namespace UE

#undef UE_API
