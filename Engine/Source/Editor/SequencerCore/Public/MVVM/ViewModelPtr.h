// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Misc/AssertionMacros.h"
#include "Misc/GeneratedTypeName.h"
#include "SequencerCoreFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"

namespace UE::Sequencer
{

/**
 * Storage selector implemented as a type alias to defer instantiation and allow use with fwd declared types
 */
template<typename T>
struct TViewModelPtrStorage
{
	/*~ Define either 'FViewModel' or 'const FViewModel' based on the constness of T */
	using ViewModelType = typename std::conditional_t<std::is_const_v<T>, const FViewModel, FViewModel>;

	FORCEINLINE T* Get() const
	{
		return Extension;
	}
	FORCEINLINE const TSharedPtr<ViewModelType>& GetModel() const
	{
		return SharedModel;
	}

	void Set(TSharedPtr<ViewModelType> InModel, T* ExtensionPtr)
	{
		if (ExtensionPtr)
		{
			SharedModel = InModel;
			Extension = ExtensionPtr;
		}
		else
		{
			SharedModel = nullptr;
			Extension = nullptr;
		}
	}

	TSharedPtr<ViewModelType> SharedModel;
	T* Extension = nullptr;
};

template<typename T>
struct TViewModelConversions
{
	/*~ Define either 'FViewModel' or 'const FViewModel' based on the constness of T */
	using ViewModelType = typename std::conditional_t<std::is_const_v<T>, const FViewModel, FViewModel>;

	operator TSharedPtr<ViewModelType>() const
	{
		return static_cast<const TViewModelPtr<T>*>(this)->Storage.GetModel();
	}
	operator TSharedPtr<T>() const
	{
		const TViewModelPtr<T>* This = static_cast<const TViewModelPtr<T>*>(this);
		return TSharedPtr<T>(This->Storage.SharedModel, This->Storage.Extension);
	}
	operator TWeakPtr<ViewModelType>() const
	{
		return TSharedPtr<ViewModelType>(*this);
	}
	operator TWeakPtr<T>() const
	{
		return TSharedPtr<T>(*this);
	}
	TWeakViewModelPtr<T> AsWeak() const
	{
		return *static_cast<const TViewModelPtr<T>*>(this);
	}
};
template<>
struct TViewModelConversions<FViewModel>
{
	SEQUENCERCORE_API operator TSharedPtr<FViewModel>() const;
	SEQUENCERCORE_API operator TWeakPtr<FViewModel>() const;

	SEQUENCERCORE_API TWeakViewModelPtr<FViewModel> AsWeak() const;
};
template<>
struct TViewModelConversions<const FViewModel>
{
	SEQUENCERCORE_API operator TSharedPtr<const FViewModel>() const;
	SEQUENCERCORE_API operator TWeakPtr<const FViewModel>() const;

	SEQUENCERCORE_API TWeakViewModelPtr<const FViewModel> AsWeak() const;
};

/**
 * Pointer type that wraps a shared view model pointer to provide implicit dynamic_cast style
 *    casting to any other view-model or extension. Implicit conversion is implemented through
 *    a proxy object so as to make the 'implicit' conversion somewhat explicit:
 *         FViewModelPtr ModelPtr = GetViewModel();
 *         TSharedPtr<IDraggableOutlinerExtension>   Draggable = ModelPtr.ImplicitCast();
 *         TSharedPtr<IDroppableExtension>           Droppable = ModelPtr.ImplicitCast();
 */
template<typename T>
struct TViewModelPtr : TViewModelConversions<T>
{
	using ViewModelType = typename std::conditional_t<std::is_const_v<T>, const FViewModel, FViewModel>;

	TViewModelPtr() = default;
	TViewModelPtr(const TViewModelPtr&) = default;
	TViewModelPtr& operator=(const TViewModelPtr&) = default;
	TViewModelPtr(TViewModelPtr&&) = default;
	TViewModelPtr& operator=(TViewModelPtr&&) = default;

	explicit TViewModelPtr(T* InViewModelType)
	{
		Storage.Set(InViewModelType->AsShared(), InViewModelType);
	}

	TViewModelPtr(TSharedPtr<ViewModelType> InModel, T* InExtension)
	{
		if (InExtension)
		{
			Storage.Set(InModel, InExtension);
		}
	}

	template<typename OtherType>
	TViewModelPtr(const TViewModelPtr<OtherType>& In)
	{
		if constexpr (TPointerIsConvertibleFromTo<OtherType, T>::Value)
		{
			Storage.Set(In.Storage.SharedModel, In.Storage.Get());
		}
		else
		{
			Storage.Set(In.Storage.SharedModel, In.Storage.SharedModel.Get());
		}
	}

	template<typename OtherType>
	TViewModelPtr& operator=(const TViewModelPtr<OtherType>& In)
	{
		if constexpr (TPointerIsConvertibleFromTo<OtherType, T>::Value)
		{
			Storage.Set(In.Storage.SharedModel, In.Storage.Get());
		}
		else
		{
			Storage.Set(In.Storage.SharedModel, In.Storage.SharedModel.Get());
		}
		return *this;
	}

	/** Construction from a nullptr */
	TViewModelPtr(TYPE_OF_NULLPTR)
	{}

	/** Assignment from a nullptr */
	TViewModelPtr& operator=(TYPE_OF_NULLPTR)
	{
		Storage.Set(nullptr, nullptr);
		return *this;
	}

	/** Construction from a shared ptr */
	template<typename ModelType>
	TViewModelPtr(const TSharedPtr<ModelType>& InModel)
	{
		static_assert(TPointerIsConvertibleFromTo<ModelType, T>::Value && TPointerIsConvertibleFromTo<ModelType, ViewModelType>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		Storage.Set(InModel, InModel.Get());
	}
	/** Assignment from a shared ptr */
	template<typename ModelType>
	TViewModelPtr& operator=(const TSharedPtr<ModelType>& InModel)
	{
		static_assert(TPointerIsConvertibleFromTo<ModelType, T>::Value && TPointerIsConvertibleFromTo<ModelType, ViewModelType>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		Storage.Set(InModel, InModel.Get());
		return *this;
	}

	/** Construction from a shared ptr */
	template<typename ModelType>
	TViewModelPtr(const TSharedRef<ModelType>& InModel)
	{
		static_assert(TPointerIsConvertibleFromTo<ModelType, T>::Value && TPointerIsConvertibleFromTo<ModelType, ViewModelType>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		Storage.Set(TSharedPtr<ModelType>(InModel), &InModel.Get());
	}

	/** Assignment from a shared ptr */
	template<typename ModelType>
	TViewModelPtr& operator=(const TSharedRef<ModelType>& InModel)
	{
		static_assert(TPointerIsConvertibleFromTo<ModelType, T>::Value && TPointerIsConvertibleFromTo<ModelType, ViewModelType>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		Storage.Set(TSharedPtr<ModelType>(InModel), &InModel.Get());
		return *this;
	}


public:

	/**
	 * Check whether this ptr is valid (non-null)
	 */
	explicit operator bool() const
	{
		return Storage.SharedModel.IsValid();
	}

	/**
	 * Reset this view model pointer back to its default (null) state
	 */
	void Reset()
	{
		Storage.Set(nullptr, nullptr);
	}

	/**
	 * Member access operator giving access to the wrapped view model
	 */
	T* operator->() const
	{
		T* Ptr = Storage.Get();
		checkf(Ptr, TEXT("Attempting to access a nullptr."));
		return Ptr;
	}

	/**
	 * Member access operator giving access to the wrapped view model
	 */
	T* Get() const
	{
		return Storage.Get();
	}

	/**
	 * Dereference operator giving access to the wrapped view model
	 */
	T& operator*() const
	{
		T* Ptr = Storage.Get();
		checkf(Ptr, TEXT("Attempting to access a nullptr."));
		return *Ptr;
	}

	/**
	 * Explicitly access the view model as a regular shared pointer
	 */
	TSharedPtr<ViewModelType> AsModel() const
	{
		return Storage.SharedModel;
	}
	TSharedRef<T> ToSharedRef() const
	{
		return TSharedPtr<T>(*this).ToSharedRef();
	}
	bool IsValid() const
	{
		return Storage.Extension != nullptr;
	}

	/**
	 * Return a proxy type that is able to implicitly convert this view
	 * model to any other TViewModel<IMyExtension> or TSharedPtr<IMyExtension>
	 */
	TImplicitViewModelCast<ViewModelType> ImplicitCast() const;
	TImplicitViewModelCastChecked<ViewModelType> ImplicitCastChecked() const;

	/**
	 * Type hash function for use with the TSet and TMap family of containers
	 */
	friend uint32 GetTypeHash(const TViewModelPtr<T>& In)
	{
		return GetTypeHash(In.Storage.SharedModel);
	}

	/**
	 * Equality operator comparing 2 view model pointers
	 */
	friend bool operator==(const TViewModelPtr<T>& A, const TViewModelPtr<T>& B)
	{
		return A.Storage.SharedModel == B.Storage.SharedModel;
	}
	friend bool operator!=(const TViewModelPtr<T>& A, const TViewModelPtr<T>& B)
	{
		return A.Storage.SharedModel != B.Storage.SharedModel;
	}

	template<typename SharedType>
	friend bool operator==(const TViewModelPtr<T>& A, const TSharedPtr<SharedType>& B)
	{
		return A.Storage.SharedModel == B;
	}
	template<typename SharedType>
	friend bool operator!=(const TViewModelPtr<T>& A, const TSharedPtr<SharedType>& B)
	{
		return A.Storage.SharedModel != B;
	}

	template<typename SharedType>
	friend bool operator==(const TViewModelPtr<T>& A, const TSharedRef<SharedType>& B)
	{
		return A.Storage.SharedModel == B;
	}
	template<typename SharedType>
	friend bool operator!=(const TViewModelPtr<T>& A, const TSharedRef<SharedType>& B)
	{
		return A.Storage.SharedModel != B;
	}

	friend bool operator==(const TViewModelPtr<T>& A, TYPE_OF_NULLPTR B)
	{
		return A.Storage.SharedModel == nullptr;
	}
	friend bool operator!=(const TViewModelPtr<T>& A, TYPE_OF_NULLPTR B)
	{
		return A.Storage.SharedModel != nullptr;
	}
protected:
	template<typename>
	friend struct TViewModelPtr;

	friend TViewModelConversions<T>;

	TViewModelPtrStorage<T> Storage;
};


template<typename OutModelType, typename InModelType>
TViewModelPtr<OutModelType> CastViewModel(const TSharedRef<InModelType>& InViewModel)
{
	return TViewModelPtr<OutModelType>(InViewModel, InViewModel->template CastThis<OutModelType>());
}
template<typename OutModelType, typename InModelType>
TViewModelPtr<OutModelType> CastViewModel(const TSharedPtr<InModelType>& InViewModel)
{
	if (!InViewModel)
	{
		return nullptr;
	}
	return TViewModelPtr<OutModelType>(InViewModel, InViewModel->template CastThis<OutModelType>());
}
template<typename OutModelType, typename InModelType>
TViewModelPtr<OutModelType> CastViewModel(const TViewModelPtr<InModelType>& InViewModel)
{
	return InViewModel.ImplicitCast();
}

template<typename OutModelType, typename InModelType>
TViewModelPtr<OutModelType> CastViewModelChecked(const TSharedRef<InModelType>& InViewModel)
{
	OutModelType* Extension = InViewModel->template CastThis<OutModelType>();
	check(Extension);
	return TViewModelPtr<OutModelType>(InViewModel, Extension);
}
template<typename OutModelType, typename InModelType>
TViewModelPtr<OutModelType> CastViewModelChecked(const TSharedPtr<InModelType>& InViewModel)
{
	OutModelType* Extension = InViewModel ? InViewModel->template CastThis<OutModelType>() : nullptr;
	check(Extension);
	return TViewModelPtr<OutModelType>(InViewModel, Extension);
}
template<typename OutModelType, typename InModelType>
TViewModelPtr<OutModelType> CastViewModelChecked(const TViewModelPtr<InModelType>& InViewModel)
{
	TViewModelPtr<OutModelType> Result = InViewModel.ImplicitCast();
	check(Result);
	return Result;
}

/**
 * Weak pointer utility for view models that also provides the same explicit conversion utility as F/TViewModelPtr
 * Using this pointer type allows for expressive null handling and casting in a simple function call for any
 * extension type which is useful for many widgets:
 *
 *    class SMyWidget
 *    {
 *        void OnMouseMove()
 *        {
 *            TViewModelPtr<IDraggable> Draggable = WeakViewModel.ImplicitPin();
 *            if (Draggable)
 *            {
 *                Draggable->BeginDrag();
 *            }
 *        }
 *        FWeakViewModelPtr WeakViewModel;
 *    };
 */

template<typename T>
struct TWeakViewModelConversions
{
	using ViewModelType = typename std::conditional_t<std::is_const_v<T>, const FViewModel, FViewModel>;

	operator TWeakPtr<ViewModelType>() const
	{
		return static_cast<const TWeakViewModelPtr<T>*>(this)->WeakModel;
	}
};

/**
 * Templated version of FWeakViewModelPtr that provides an additional self-documenting extension that should
 * exist on the view model.
 */
template<typename T>
struct TWeakViewModelPtr : TWeakViewModelConversions<T>
{
	using ViewModelType = typename std::conditional_t<std::is_const_v<T>, const FViewModel, FViewModel>;

	TWeakViewModelPtr() = default;
	TWeakViewModelPtr(const TWeakViewModelPtr&) = default;
	TWeakViewModelPtr& operator=(const TWeakViewModelPtr&) = default;
	TWeakViewModelPtr(TWeakViewModelPtr&&) = default;
	TWeakViewModelPtr& operator=(TWeakViewModelPtr&&) = default;

	TWeakViewModelPtr(TYPE_OF_NULLPTR)
	{}

	template<typename OtherType>
	TWeakViewModelPtr(const TWeakPtr<OtherType>& InModel)
	{
		static_assert(TPointerIsConvertibleFromTo<OtherType, T>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		WeakModel = InModel;
	}
	template<typename OtherType>
	TWeakViewModelPtr(const TSharedPtr<OtherType>& InModel)
	{
		static_assert(TPointerIsConvertibleFromTo<OtherType, T>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		WeakModel = InModel;
	}
	template<typename OtherType>
	TWeakViewModelPtr(const TSharedRef<OtherType>& InModel)
	{
		static_assert(TPointerIsConvertibleFromTo<OtherType, T>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		WeakModel = InModel;
	}
	/** Construction from a TViewModelPtr */
	template<typename OtherType>
	TWeakViewModelPtr(const TViewModelPtr<OtherType>& InModel)
	{
		static_assert(std::is_same_v<T, ViewModelType> || TPointerIsConvertibleFromTo<OtherType, T>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		WeakModel = InModel.AsModel();
	}
	/** Assignment from a TViewModelPtr */
	template<typename OtherType>
	TWeakViewModelPtr& operator=(const TViewModelPtr<OtherType>& InModel)
	{
		static_assert(std::is_same_v<T, ViewModelType> || TPointerIsConvertibleFromTo<OtherType, T>::Value,
			"Construction from shared pointers is only supported for related types. Please use CastViewModel<> to cast to the correct type.");

		WeakModel = InModel.AsModel();
		return *this;
	}

	/** Pin this weak view model ptr */
	TViewModelPtr<T> Pin() const
	{
		TSharedPtr<ViewModelType> Model = WeakModel.Pin();
		return CastViewModel<T>(Model);
	}

	/**
	 * Returns a proxy object that will simultaneously Pin() and Cast() this view model
	 * to a particular view-model or extension type
	 */
	TImplicitWeakViewModelPin<ViewModelType> ImplicitPin() const;

	friend uint32 GetTypeHash(const TWeakViewModelPtr<T>& In)
	{
		return GetTypeHash(In.WeakModel);
	}

	friend bool operator==(const TWeakViewModelPtr<T>& A, const TWeakViewModelPtr<T>& B)
	{
		return A.WeakModel == B.WeakModel;
	}

	friend bool operator!=(const TWeakViewModelPtr<T>& A, const TWeakViewModelPtr<T>& B)
	{
		return A.WeakModel != B.WeakModel;
	}
protected:

	friend TWeakViewModelConversions<T>;

	TWeakPtr<ViewModelType> WeakModel;
};


/**
 * Proxy type that enables implicit dynamic casting of a view-model to any other view-model
 * or extension type. This is implemented as a proxy type rather than the conversion
 * operators existing on FViewModelPtr so that the implicit cast must be user-directed, while
 * still removing additional template boilerplate at the call-site.
 *
 * @see FViewModelPtr::ImplicitCast
 */
template<typename ViewModelType, bool bChecked>
struct TImplicitViewModelCastImpl
{
	/** Construction from a regular shared ptr to a view model */
	explicit TImplicitViewModelCastImpl(const TSharedPtr<ViewModelType>& In)
		: ViewModel(In)
	{}

	/**
	 * Implicitly convert this view model to any other view-model or extension type
	 * @return A regular shared pointer to the view-model or extension if it is implemented on the existing model; nullptr otherwise.
	 */
	template<typename ExtensionType>
	operator TSharedPtr<ExtensionType>() const
	{
		using FinalExtensionType = std::conditional_t<std::is_const_v<ViewModelType>, const ExtensionType, ExtensionType>;

		TViewModelPtr<FinalExtensionType> Result = ViewModel ? ViewModel->template CastThisShared<FinalExtensionType>() : nullptr;
		if constexpr (bChecked)
		{
			checkf(Result, TEXT("Failed to cast view model"));
		}
		return Result;
	}

	/**
	 * Implicitly convert this view model to any other view-model or extension type
	 * @return A TViewModelPtr to the view-model or extension if it is implemented on the existing model; nullptr otherwise.
	 */
	template<typename ExtensionType>
	operator TViewModelPtr<ExtensionType>() const
	{
		using FinalExtensionType = std::conditional_t<std::is_const_v<ViewModelType>, const ExtensionType, ExtensionType>;

		TViewModelPtr<FinalExtensionType> Result = ViewModel ? ViewModel->template CastThisShared<ExtensionType>() : nullptr;
		if constexpr (bChecked)
		{
			checkf(Result, TEXT("Failed to cast view model"));
		}
		return Result;
	}

	/**
	 * Implicitly convert this view model to any other view-model or extension type
	 * @return A TWeakViewModelPtr to the view-model or extension if it is implemented on the existing model; nullptr otherwise.
	 */
	template<typename ExtensionType>
	operator TWeakViewModelPtr<ExtensionType>() const
	{
		return TViewModelPtr<ExtensionType>(*this);
	}

private:

	TSharedPtr<ViewModelType> ViewModel;
};

/**
 * Proxy type that combines a Pin() and Cast() of a weak view model ptr, gracefully handling null.
 *
 * @see FWeakViewModelPtr::ImplicitPin
 */
template<typename ViewModelType>
struct TImplicitWeakViewModelPin
{
	/** Construction from a regular weak ptr to a view model */
	explicit TImplicitWeakViewModelPin(const TWeakPtr<ViewModelType>& In)
		: WeakModel(In)
	{}

	/**
	 * Simultaneously Pin() and Cast() this view model to the templated type
	 * @return A shared pointer to the view-model or extension type if it is implemented on this model, nullptr otherwise.
	 */
	template<typename ExtensionType>
	operator TSharedPtr<ExtensionType>() const
	{
		using FinalExtensionType = std::conditional_t<std::is_const_v<ViewModelType>, const ExtensionType, ExtensionType>;
		TSharedPtr<ViewModelType> Pinned = WeakModel.Pin();
		if (Pinned)
		{
			return TSharedPtr<ExtensionType>(Pinned, Pinned->template CastThis<ExtensionType>());
		}
		return nullptr;
	}

	/**
	 * Simultaneously Pin() and Cast() this view model to the templated type
	 * @return A TViewModelPtr to the view-model or extension type if it is implemented on this model, nullptr otherwise.
	 */
	template<typename ExtensionType>
	operator TViewModelPtr<ExtensionType>() const
	{
		using FinalExtensionType = std::conditional_t<std::is_const_v<ViewModelType>, const ExtensionType, ExtensionType>;

		TSharedPtr<ViewModelType> Pinned = WeakModel.Pin();
		if (Pinned)
		{
			return TViewModelPtr<FinalExtensionType>(Pinned, Pinned->template CastThis<FinalExtensionType>());
		}
		return nullptr;
	}

	template<typename ExtensionType>
	operator TWeakViewModelPtr<ExtensionType>() const
	{
		TViewModelPtr<ExtensionType> Pinned = *this;
		return Pinned;
	}
private:

	TWeakPtr<ViewModelType> WeakModel;
};


template<typename T>
TImplicitViewModelCast<typename TViewModelPtr<T>::ViewModelType> TViewModelPtr<T>::ImplicitCast() const
{
	return TImplicitViewModelCast<ViewModelType>{Storage.SharedModel};
}
template<typename T>
TImplicitViewModelCastChecked<typename TViewModelPtr<T>::ViewModelType> TViewModelPtr<T>::ImplicitCastChecked() const
{
	return TImplicitViewModelCastChecked<ViewModelType>{Storage.SharedModel};
}

template<typename T>
TImplicitWeakViewModelPin<typename TWeakViewModelPtr<T>::ViewModelType> TWeakViewModelPtr<T>::ImplicitPin() const
{
	return TImplicitWeakViewModelPin<ViewModelType>{WeakModel};
}

} // namespace UE::Sequencer
