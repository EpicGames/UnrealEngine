// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/CastableTypeTable.h"

#define UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID(Type)																	\
	static ::UE::Sequencer::TAutoRegisterViewModelTypeID<Type> ID;														\
	static void RegisterTypeID();

#define UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(MODULE_API, Type)													\
	MODULE_API static ::UE::Sequencer::TAutoRegisterViewModelTypeID<Type> ID;											\
	MODULE_API static void RegisterTypeID();

#define UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(Type)																	\
	::UE::Sequencer::TAutoRegisterViewModelTypeID<Type> Type::ID;														\
	void Type::RegisterTypeID()																							\
	{																													\
		Type::ID.ID        = ::UE::Sequencer::FViewModelTypeID::RegisterNewID();										\
		Type::ID.TypeTable = ::UE::Sequencer::FCastableTypeTable::MakeTypeTable<Type>((Type*)0, Type::ID.ID, #Type);	\
	}

namespace UE
{
namespace Sequencer
{

struct FCastableTypeTable;

struct FViewModelTypeID
{
	friend uint32 GetTypeHash(FViewModelTypeID In)
	{
		return In.ID;
	}

	friend bool operator<(FViewModelTypeID A, FViewModelTypeID B)
	{
		return A.ID < B.ID;
	}

	friend bool operator==(FViewModelTypeID A, FViewModelTypeID B)
	{
		return A.ID == B.ID;
	}

	uint32 GetTypeID() const
	{
		return ID;
	}

	SEQUENCERCORE_API static uint32 RegisterNewID();

	FViewModelTypeID(FCastableTypeTable* InTypeTable, uint32 InID)
		: TypeTable(InTypeTable)
		, ID(InID)
	{}

private:

	FCastableTypeTable* TypeTable;
	uint32 ID;
};

template<typename T>
struct TViewModelTypeID
{
	operator FViewModelTypeID() const
	{
		Register();
		return FViewModelTypeID{ TypeTable, ID };
	}

	uint32 GetTypeID() const
	{
		Register();
		return ID;
	}

	FCastableTypeTable* GetTypeTable() const
	{
		Register();
		return TypeTable;
	}

	void Register() const
	{
		if (!IsRegistered())
		{
			T::RegisterTypeID();
		}
	}

protected:
	friend T;

	bool IsRegistered() const
	{
		return ID != ~0u;
	}

	FCastableTypeTable* TypeTable = nullptr;
	uint32 ID = ~0u;
};

template<typename T>
struct TAutoRegisterViewModelTypeID : TViewModelTypeID<T>
{
};

} // namespace Sequencer
} // namespace UE

