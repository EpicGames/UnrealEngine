// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/EditorViewModel.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

class FEditorSharedViewModelData
	: public FSharedViewModelData
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FEditorSharedViewModelData, FSharedViewModelData);

	FEditorSharedViewModelData(TSharedRef<FEditorViewModel> InEditor)
		: WeakEditor(InEditor)
	{
	}

	TSharedPtr<FEditorViewModel> GetEditor() const
	{
		return WeakEditor.Pin();
	}

private:

	/** The editor view model for this shared data */
	TWeakPtr<FEditorViewModel> WeakEditor;
};

} // namespace UE::Sequencer

#undef UE_API
