// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"
#include "TimeToPixel.h"
#include "UObject/StrongObjectPtr.h"

#define UE_API SEQUENCERCORE_API

class USequencerScriptingLayer;

namespace UE::Sequencer
{

class FOutlinerViewModel;
class FTrackAreaViewModel;
class FSequencerCoreSelection;

template <typename T> struct TAutoRegisterViewModelTypeID;

/**
 * This represents to root view-model for a sequencer-like editor.
 *
 * This view-model assumes that the editor includes at least an outliner area and a track area.
 * Other panels in the editor can be added to the list of panels.
 *
 * The data being edited is represented by a root view-model which doesn't change: if the editor
 * needs to edit a different piece of data, this should change *inside* the root view-model, and
 * call the necessary code to refresh the entire view-model hierarchy.
 * Note that this root view-model is *not* parented under the editor view-model.
 */
class FEditorViewModel
	: public FViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FEditorViewModel, FViewModel);

	/** Children type for editor panels */
	static UE_API EViewModelListType GetEditorPanelType();

	/** Builds a new editor view model. */
	UE_API FEditorViewModel();
	/** Destroys this editor view model. */
	UE_API ~FEditorViewModel();

	/** Initializes the outliner and track-area for this editor. */
	UE_API void InitializeEditor();

	/** Gets the root view-model of the sequence being edited. */
	UE_API FViewModelPtr GetRootModel() const;

	/** Gets the editor panels' view models. */
	UE_API FViewModelChildren GetEditorPanels();

	/** Gets the outliner view-model. */
	UE_API TSharedPtr<FOutlinerViewModel>  GetOutliner() const;

	/** Gets the track area view-model. */
	UE_API TSharedPtr<FTrackAreaViewModel> GetTrackArea() const;

	/** Gets the selection class */
	UE_API TSharedPtr<FSequencerCoreSelection> GetSelection() const;

	/** Gets the scripting layer */
	UE_API USequencerScriptingLayer* GetScriptingLayer() const;

	/** Returns the current view density for this editor */
	UE_API FViewDensityInfo GetViewDensity() const;

	/** Returns the current view density for this editor */
	UE_API void SetViewDensity(const FViewDensityInfo& InViewDensity);

	/** Returns whether this editor is currently read-only */
	UE_API virtual bool IsReadOnly() const;

	/** Returns the inverse of read-only - useful for direct bindings to IsEnabled for widgets */
	bool IsEditable() const
	{
		return !IsReadOnly();
	}

protected:

	/** Initializes this view model before panels and root models are created */
	virtual void PreInitializeEditorImpl() {}
	/** Creates the root data model for this editor. */
	UE_API virtual TSharedPtr<FViewModel> CreateRootModelImpl();
	/** Creates the outliner for this editor. */
	UE_API virtual TSharedPtr<FOutlinerViewModel> CreateOutlinerImpl();
	/** Creates the track-area for this editor. */
	UE_API virtual TSharedPtr<FTrackAreaViewModel> CreateTrackAreaImpl();
	/** Creates the selection class for this editor. */
	UE_API virtual TSharedPtr<FSequencerCoreSelection> CreateSelectionImpl();
	/** Creates the scripting layer class for this editor. */
	UE_API virtual USequencerScriptingLayer* CreateScriptingLayerImpl();
	/** Creates any other panels for this editor. */
	virtual void InitializeEditorImpl() {}

private:

	friend class FOutlinerViewModel;

	/** List of panel view-models for this editor */
	FViewModelListHead PanelList;

	/** Cached pointer to the outliner panel view-model */
	TSharedPtr<FOutlinerViewModel>  Outliner;
	/** Cached pointer to the track area panel view-model */
	TSharedPtr<FTrackAreaViewModel> TrackArea;
	/** Cached pointer to the selection class */
	TSharedPtr<FSequencerCoreSelection> Selection;
	/** Cached pointer to the scripting layer class */
	TStrongObjectPtr<USequencerScriptingLayer> ScriptingLayer;

	/** The root view-model for the data being edited */
	TSharedPtr<FViewModel> RootDataModel;

	/** This editor's current view density defining how condensed the elements appear */
	FViewDensityInfo ViewDensity;
};

} // namespace UE::Sequencer

#undef UE_API
