// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "MVVM/ViewModelPtr.h"
#include "Templates/SubclassOf.h"

class SWidget;
class UMovieSceneSequence;

namespace UE::Sequencer 
{

struct FOutlinerColumnPosition;
struct FOutlinerColumnLayout;

class ISequencerTreeViewRow;
class IOutlinerExtension;
class FEditorViewModel;


/** Parameters for creating an outliner column widget. */
struct FCreateOutlinerColumnParams
{
	FCreateOutlinerColumnParams(const TViewModelPtr<IOutlinerExtension>& InOutlinerExtension, const TSharedPtr<FEditorViewModel> InEditor)
		: OutlinerExtension(InOutlinerExtension)
		, Editor(InEditor)
	{}

	const TViewModelPtr<IOutlinerExtension> OutlinerExtension;
	const TSharedPtr<FEditorViewModel> Editor;
};

/**
* Interface for sequencer outliner columns.
*/
class IOutlinerColumn : public TSharedFromThis<IOutlinerColumn>
{

public:

	/** Returns the name of the column. Used for determining column type when dragging or saving settings. */
	virtual FName GetColumnName() const = 0;

	/** Returns the label of the column to display in visibility settings. */
	virtual FText GetColumnLabel() const = 0;

	/** Get this columns position data relative to other columns */
	virtual FOutlinerColumnPosition GetPosition() const = 0;

	/** Get the layout information for this column's cells */
	virtual FOutlinerColumnLayout GetLayout() const = 0;

	/* The default visibility state of this column when loaded for the first time. */
	virtual bool IsColumnVisibleByDefault() const { return true; }

	/* Gets whether or not this column is supported by a given Sequencer. */
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const { return true; }

	/* Gets whether or not a widget should be generated for a given item in the outliner column. */
	virtual bool IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const = 0;

	/* Gets the widget created for each item within the SOutlinerView, column widgets must be fixed width. */
	virtual TSharedPtr<SWidget> CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow) = 0;

public:

	/** Virtual destructor. */
	virtual ~IOutlinerColumn() = default;

};


} // namespace UE::Sequencer
