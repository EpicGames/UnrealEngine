// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnBase.h"

namespace UE::Sequencer
{

/**
 * Label Column
 */
class FLabelOutlinerColumn
	: public FOutlinerColumnBase
{
public:

	SEQUENCERCORE_API FLabelOutlinerColumn();

	TSharedPtr<SWidget> CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow) override;
};

} // namespace UE::Sequencer