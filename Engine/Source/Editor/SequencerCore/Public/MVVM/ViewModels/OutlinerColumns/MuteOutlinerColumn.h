// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnBase.h"

namespace UE::Sequencer
{

/**
 * A column for muting/unmuting tracks.
 */
class FMuteOutlinerColumn
	: public FOutlinerColumnBase
{
public:

	SEQUENCERCORE_API FMuteOutlinerColumn();

	bool IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const override;
	TSharedPtr<SWidget> CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow) override;
};


} // namespace UE::Sequencer