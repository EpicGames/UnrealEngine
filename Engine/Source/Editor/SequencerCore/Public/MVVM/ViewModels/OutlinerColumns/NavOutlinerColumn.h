// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnBase.h"

namespace UE::Sequencer
{

class FNavOutlinerColumn
	: public FOutlinerColumnBase
{
public:

	SEQUENCERCORE_API FNavOutlinerColumn();
};

} // namespace UE::Sequencer