// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"

namespace UE::Sequencer
{

class FOutlinerColumnBase
	: public IOutlinerColumn
{
public:

	FOutlinerColumnBase()
	{
	}

	FOutlinerColumnBase(FName InName, FText InLabel, FOutlinerColumnPosition InPosition, FOutlinerColumnLayout InLayout)
		: Name(InName), Label(InLabel), Position(InPosition), Layout(InLayout)
	{
	}

	FName GetColumnName() const override
	{
		return Name;
	}

	FText GetColumnLabel() const override
	{
		return Label;
	}

	FOutlinerColumnPosition GetPosition() const override
	{
		return Position;
	}

	FOutlinerColumnLayout GetLayout() const override
	{
		return Layout;
	}

	bool IsColumnVisibleByDefault() const override
	{
		return true;
	}

	bool IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const override
	{
		return true;
	}

	TSharedPtr<SWidget> CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow) override
	{
		return nullptr;
	}

protected:

	FName Name;
	FText Label;
	FOutlinerColumnPosition Position;
	FOutlinerColumnLayout Layout;
};

} // namespace UE::Sequencer