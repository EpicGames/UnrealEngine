// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Layout/Margin.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Sequencer
{

enum class EOutlinerColumnFlags : uint8
{
	None = 0,

	OverflowSubsequentEmptyCells = 1 << 0,

	OverflowOnHover = 1 << 1,

	Hidden = 1 << 2,
};
ENUM_CLASS_FLAGS(EOutlinerColumnFlags)

enum class EOutlinerColumnGroup : uint8
{
	/** Far left gutter area: Indicators */
	LeftGutter,

	/** Asset state toggles area: Pin, Lock, Deactivate (states saved to the asset) */
	AssetToggles,

	/** Editor transient state toggles area: Mute, Solo (states NOT saved to the asset) */
	TransientToggles,

	/** Main center area: Add, Edit, Label */
	Center,

	/** Far right gutter area: Key Frame, Key Nav, Color Picker, etc. */
	RightGutter,
};

enum class EOutlinerColumnSizeMode : uint8
{
	Fixed,
	Stretch,
};

struct FOutlinerColumnPosition
{
	int16 SortOrder = 0;
	EOutlinerColumnGroup Group = EOutlinerColumnGroup::Center;

	friend bool operator<(FOutlinerColumnPosition A, FOutlinerColumnPosition B)
	{
		if (A.Group == B.Group)
		{
			return A.SortOrder < B.SortOrder;
		}
		return A.Group < B.Group;
	}
};

struct FOutlinerColumnLayout
{
	float Width;
	FMargin CellPadding;
	EHorizontalAlignment HAlign;
	EVerticalAlignment VAlign;
	EOutlinerColumnSizeMode SizeMode;
	EOutlinerColumnFlags Flags;
};

struct FCommonOutlinerNames
{
	static SEQUENCERCORE_API FName Indicator;
	static SEQUENCERCORE_API FName Pin;
	static SEQUENCERCORE_API FName Lock;
	static SEQUENCERCORE_API FName Deactivate;
	static SEQUENCERCORE_API FName Mute;
	static SEQUENCERCORE_API FName Solo;
	static SEQUENCERCORE_API FName Label;
	static SEQUENCERCORE_API FName Edit;
	static SEQUENCERCORE_API FName Add;
	static SEQUENCERCORE_API FName Nav;
	static SEQUENCERCORE_API FName KeyFrame;
	static SEQUENCERCORE_API FName ColorPicker;
	static SEQUENCERCORE_API FName TimeWarp;
	static SEQUENCERCORE_API FName Condition;
};


} // namespace UE::Sequencer