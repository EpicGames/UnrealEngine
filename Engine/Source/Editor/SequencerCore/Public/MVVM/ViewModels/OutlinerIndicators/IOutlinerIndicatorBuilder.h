// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWidget;
class UMovieSceneSequence;

namespace UE::Sequencer
{

struct FCreateOutlinerColumnParams;
class IOutlinerColumn;
class ISequencerTreeViewRow;

/**
* Interface for building sequencer indicator outliner items.
*/
class IOutlinerIndicatorBuilder : public TSharedFromThis<IOutlinerIndicatorBuilder>
{

public:

	/** Returns the name of the indicator outliner item. */
	virtual FName GetIndicatorName() const = 0;

	/* Gets whether or not this indicator outliner item is supported by a given Sequencer. */
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const { return true; }

	virtual bool IsItemCompatibleWithIndicator(const FCreateOutlinerColumnParams& InParams) const = 0;
	virtual TSharedPtr<SWidget> CreateIndicatorWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow, const TSharedRef<IOutlinerColumn>& OutlinerColumn, const int32 NumCompatibleIndicators) = 0;

public:

	/** Virtual destructor. */
	virtual ~IOutlinerIndicatorBuilder() = default;

};

} // namespace UE::Sequencer

