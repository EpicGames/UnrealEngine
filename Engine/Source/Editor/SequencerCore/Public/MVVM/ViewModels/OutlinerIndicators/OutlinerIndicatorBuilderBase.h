// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerIndicators/IOutlinerIndicatorBuilder.h"

namespace UE::Sequencer
{

class FOutlinerIndicatorBuilderBase
	: public IOutlinerIndicatorBuilder
{
public:

	FOutlinerIndicatorBuilderBase()
	{
	}
	
public:

	bool IsItemCompatibleWithIndicator(const FCreateOutlinerColumnParams& InParams) const override { return false; }
	TSharedPtr<SWidget> CreateIndicatorWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow, const TSharedRef<IOutlinerColumn>& OutlinerColumn, const int32 NumCompatibleIndicators) override { return nullptr; }
};

} // namespace UE::Sequencer