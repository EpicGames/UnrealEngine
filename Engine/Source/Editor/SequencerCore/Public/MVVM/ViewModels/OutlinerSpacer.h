// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/STableRow.h"

#define UE_API SEQUENCERCORE_API

class FDragDropEvent;
class SWidget;
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }

namespace UE
{
namespace Sequencer
{

class FOutlinerSpacer
	: public FViewModel
	, public FGeometryExtensionShim
	, public FOutlinerExtensionShim
	, public ISortableExtension
	, public IOutlinerDropTargetOutlinerExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FOutlinerSpacer, FViewModel, IGeometryExtension, IOutlinerExtension, ISortableExtension, IOutlinerDropTargetOutlinerExtension);

	UE_API FOutlinerSpacer(float InDesiredSpacerHeight);
	UE_API ~FOutlinerSpacer();

	void SetDesiredHeight(float InDesiredSpacerHeight)
	{
		DesiredSpacerHeight = InDesiredSpacerHeight;
	}

public:

	/*~ IOutlinerExtension */
	UE_API bool HasBackground() const override;
	UE_API FName GetIdentifier() const override;
	UE_API FOutlinerSizing GetOutlinerSizing() const override;
	UE_API TSharedPtr<SWidget> CreateContextMenuWidget(const FCreateOutlinerContextMenuWidgetParams& InParams) override;

	/*~ ISortableExtension */
	UE_API virtual void SortChildren() override;
	UE_API virtual FSortingKey GetSortingKey() const override;
	UE_API virtual void SetCustomOrder(int32 InCustomOrder) override;

	/*~ IOutlinerDropTargetOutlinerExtension */
	UE_API TOptional<EItemDropZone> CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;
	UE_API void PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;

private:

	float DesiredSpacerHeight;
	int32 CustomOrder;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
