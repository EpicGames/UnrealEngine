// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Input/DragAndDrop.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

class FDragDropOperation;
class SWidget;

namespace UE::Sequencer
{

class FEditorViewModel;
class FViewModel;
class IOutlinerExtension;

class FOutlinerViewModel
	: public FViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FOutlinerViewModel, FViewModel);

	FSimpleMulticastDelegate OnRefreshed;

	/**
	 * Constructor
	 */
	UE_API FOutlinerViewModel();

	/**
	 * Destructor
	 */
	virtual ~FOutlinerViewModel() = default;

	/**
	 * Initialize the view model with a given root model
	 */
	UE_API void Initialize(const FWeakViewModelPtr& InWeakRootDataModel);

	/**
	 * Get the editor view-model provided by the creation context
	 */
	UE_API TSharedPtr<FEditorViewModel> GetEditor() const;

	/**
	 * Get the logical root item of the outliner hierarchy
	 */
	UE_API FViewModelPtr GetRootItem() const;

	/**
	 * Retrieve a flat list of the root items in this outliner model
	 */
	UE_API TArrayView<const TWeakViewModelPtr<IOutlinerExtension>> GetTopLevelItems() const;

	/**
	 * Set the single hovered model in the tree
	 */
	UE_API void SetHoveredItem(TViewModelPtr<IOutlinerExtension> InHoveredModel);

	/**
	 * Get the single hovered model in the tree, possibly nullptr
	 */
	UE_API TViewModelPtr<IOutlinerExtension> GetHoveredItem() const;

	/**
	 * Unpins any pinned nodes in this tree
	 */
	UE_API void UnpinAllNodes();

	/**
	 * Called when the default context menu is opened
	 */
	UE_API virtual TSharedPtr<SWidget> CreateContextMenuWidget();

	/**
	 * Initate a new drag operation from the specified dragged models.
	 */
	UE_API virtual TSharedRef<FDragDropOperation> InitiateDrag(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedModels);

	/**
	 * Request that this tree be updated
	 */
	UE_API virtual void RequestUpdate();

	/**
	 * Returns true if the root item hierarchy has been invalidated since the last call to GetTopLevelItems.
	 * Used by SOutlinerView::Refresh to decide between a full list rebuild (hierarchy changed) and an
	 * incremental refresh pass (state-only change such as mute/solo/deactivate).
	 */
	bool IsHierarchyInvalidated() const { return bRootItemsInvalidated; }

private:

	UE_API void HandleDataHierarchyChanged();

private:

	/** The root data */
	FWeakViewModelPtr WeakRootDataModel;
	/** A cached list of the root data's children, which are the outliner's root items */
	TArray<TWeakViewModelPtr<IOutlinerExtension>> RootItems;

	/** The currently mouse-hovered item */
	TWeakViewModelPtr<IOutlinerExtension> WeakHoveredItem;

	/** Whether root items should be re-evaluated */
	bool bRootItemsInvalidated;
};

} // namespace UE::Sequencer

#undef UE_API
