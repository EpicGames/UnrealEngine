// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GraphEditorDragDropAction.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Text.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/IDraggableTrackAreaExtension.h" // for IDragOperation
#include "MVVM/ViewModelPtr.h"
#include "Misc/FrameNumber.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

struct FSlateBrush;
enum class EItemDropZone;

namespace UE
{
namespace Sequencer
{

class FViewModel;
class IOutlinerExtension;

/** A decorated drag drop operation object for dragging sequencer outliner items. */
class FOutlinerViewModelDragDropOp 
	: public FGraphEditorDragDropAction
	, public IDragOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE( FOutlinerViewModelDragDropOp, FGraphEditorDragDropAction )

	/**
	 * Construct a new drag/drop operation for dragging a selection of display nodes
	 */
	static UE_API TSharedRef<FOutlinerViewModelDragDropOp> New(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedViewModels, FText InDefaultText, const FSlateBrush* InDefaultIcon);

protected:

	/** Protected construction for derived types. Generally constructed via a New() factory function to ensure that Construct is called */
	FOutlinerViewModelDragDropOp()
	{}

public:

	//~ FDragDropOperation interface
	UE_API virtual void Construct() override;

public:

	/**
	 * Reset the tooltip decorator back to its original state
	 */
	UE_API void ResetToDefaultToolTip();

	/**
	 * Gets the nodes which are currently being dragged.
	 */
	UE_API TArrayView<const TWeakViewModelPtr<IOutlinerExtension>> GetDraggedViewModels() const;

	/**
	 * Check whether dropping onto the specified outliner item would result in a parent being dropped onto its child
	 * @note: Sets this operation's CurrentHoverText to an error string on failure
	 */
	UE_API bool ValidateParentChildDrop(const FViewModel& ProspectiveItem);

	/*
	* Based on the current view model being hovered, return the 'target' model and the 'drop model'. 
	* DropModel is the model we will call to execute the drop. 
	* TargetModel is only set if we're inserting before or after a child model of DropModel. 
	* We default to using the CurrentViewModel for the DropModel unless we've dragged above or below the CurrentViewModel,
	* in which we default to the CurrentViewModel's parent for the DropModel and the TargetModel as the CurrentViewModel.
	*/
	UE_API virtual void GetTargetAndDropModel(const FViewModelPtr& CurrentViewModel, EItemDropZone InItemDropZone, FViewModelPtr& DropModel, FViewModelPtr& TargetModel) const;

private:

	void AddSnapTime(FFrameNumber SnapTime) override
	{}

	void AddModel(TSharedPtr<FViewModel> Model) override
	{}

	/**
	 * Get the current decorator text
	 */
	FText GetDecoratorText() const
	{
		return CurrentHoverText;
	}

	/**
	 * Get the current decorator icon
	 */
	const FSlateBrush* GetDecoratorIcon() const
	{
		return CurrentIconBrush;
	}

public:

	/**
	 * Current string to show as the decorator text
	 */
	FText CurrentHoverText;

	/**
	 * Current icon to be displayed on the decorator
	 */
	const FSlateBrush* CurrentIconBrush;

protected:

	/** The nodes currently being dragged. */
	TArray<TWeakViewModelPtr<IOutlinerExtension>> WeakViewModels; 
	
	/** Default string to show as hover text */
	FText DefaultHoverText;

	/** Default icon to be displayed */
	const FSlateBrush* DefaultHoverIcon;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
