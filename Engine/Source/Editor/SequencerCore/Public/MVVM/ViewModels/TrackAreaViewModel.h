// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Math/Range.h"
#include "Templates/SharedPointer.h"
#include "TimeToPixel.h"
#include "UObject/NameTypes.h"

#define UE_API SEQUENCERCORE_API

struct FFrameNumber;
struct FFrameRate;
struct FGeometry;

namespace UE
{
namespace Sequencer
{

class FEditorViewModel;
class FTrackAreaViewSpace;
class ISequencerEditTool;
struct ITrackAreaHotspot;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTrackAreaHotspotChanged, TSharedPtr<ITrackAreaHotspot>);

class FTrackAreaViewModel
	: public FViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FTrackAreaViewModel, FViewModel);

	UE_API FTrackAreaViewModel();
	UE_API virtual ~FTrackAreaViewModel();

public:

	DECLARE_MULTICAST_DELEGATE_TwoParams(FViewSpaceEvent, const FGuid&, TSharedPtr<FTrackAreaViewSpace>)

	FViewSpaceEvent OnViewSpaceAdded;
	FViewSpaceEvent OnViewSpaceRemoved;

	/** Get the editor view-model provided by the creation context */
	UE_API TSharedPtr<FEditorViewModel> GetEditor() const;

	/** Create a time/pixel converter for the given geometry */
	UE_API FTimeToPixel GetTimeToPixel(float TrackAreaWidth) const;

	/** Gets the current tick resolution of the editor */
	UE_API virtual FFrameRate GetTickResolution() const;
	/** Gets the current view range of this track area */
	UE_API virtual TRange<double> GetViewRange() const;

	UE_API virtual void OnConstruct() override;

	/** Get the current active hotspot */
	UE_API TSharedPtr<ITrackAreaHotspot> GetHotspot() const;

	/** Set the hotspot to something else */
	UE_API void SetHotspot(TSharedPtr<ITrackAreaHotspot> NewHotspot);
	UE_API void AddHotspot(TSharedPtr<ITrackAreaHotspot> NewHotspot);
	UE_API void RemoveHotspot(FViewModelTypeID Type);
	UE_API void ClearHotspots();

	/** Set whether the hotspot is locked and cannot be changed (ie when a menu is open) */
	UE_API void LockHotspot(bool bIsLocked);

	/** Get the callback that is fired when the current hotspot changes */
	UE_API FOnTrackAreaHotspotChanged& GetOnHotspotChangedDelegate();

	UE_API void AddEditTool(TSharedPtr<ISequencerEditTool> InNewTool);

	/** Access the currently active track area edit tool */
	ISequencerEditTool* GetEditTool() const { return EditTool.IsValid() ? EditTool.Get() : nullptr; }

	/** Check whether it's possible to activate the specified tool */
	UE_API bool CanActivateEditTool(FName Identifier) const;

	UE_API bool AttemptToActivateTool(FName Identifier);

	UE_API void LockEditTool();
	UE_API void UnlockEditTool();

	UE_API void AddViewSpace(const FGuid& ID, TSharedPtr<FTrackAreaViewSpace> ViewSpace);
	UE_API void RemoveViewSpace(const FGuid& ID);

	UE_API TSharedPtr<FTrackAreaViewSpace> FindViewSpace(const FGuid& InID) const;

protected:

	UE_API void UpdateViewSpaces();

protected:

	TMap<FGuid, TSharedPtr<FTrackAreaViewSpace>> ViewSpaces;

	/** The current hotspot that can be set from anywhere to initiate drags */
	TArray<TSharedPtr<ITrackAreaHotspot>> HotspotStack;

	/** The delegate that is fired when the current hotspot changes */
	FOnTrackAreaHotspotChanged OnHotspotChangedDelegate;

	/** The currently active edit tools on this track area */
	TArray<TSharedPtr<ISequencerEditTool>> EditTools;

	/** The currently active edit tool on this track area */
	TSharedPtr<ISequencerEditTool> EditTool;

	/** When true, prevents any other hotspot being activated */
	bool bHotspotLocked;

	/** When true, prevents any other edit tool being activated by way of CanActivateEditTool always returning true */
	bool bEditToolLocked;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
