// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "TimeToPixel.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

class FTrackAreaViewSpace
	: public FViewModel
	, public INonLinearTimeTransform
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FTrackAreaViewSpace, FViewModel);

	UE_API FTrackAreaViewSpace();
	UE_API virtual ~FTrackAreaViewSpace();

	UE_API virtual double SourceToView(double Seconds) const override;
	UE_API virtual double ViewToSource(double Source) const override;
};

} // namespace UE::Sequencer

#undef UE_API
