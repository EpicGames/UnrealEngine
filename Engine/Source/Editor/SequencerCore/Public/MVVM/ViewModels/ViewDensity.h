// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"

namespace UE::Sequencer
{

enum class EViewDensity
{
	/** Variable density where inner items are more condensed than top level or outer items */
	Variable,

	/** A compact view type with uniform heights */
	Compact,

	/** A relaxed view type with larger, uniform heights and additional information */
	Relaxed
};


struct FViewDensityInfo
{
	SEQUENCERCORE_API FViewDensityInfo();
	SEQUENCERCORE_API FViewDensityInfo(EViewDensity InViewDensity);

	SEQUENCERCORE_API static TOptional<float> GetUniformHeight(EViewDensity Density);

	TOptional<float> UniformHeight;

	EViewDensity Density;
};


} // namespace UE::Sequencer