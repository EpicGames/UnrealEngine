// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/Optional.h"
#include "SequencerCoreFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{
class FSharedViewModelData;
struct FParentFirstChildIterator;
struct FParentModelIterator;
struct FViewModelIterationState;
struct FViewModelListIterator;

/**
 * Base class for a sequencer data model. This might wrap an underlying UObject, or be a purely
 * logical construct for the sequencer UI.
 *
 * WARNING: most of the casting and hierarchy related methods are not safe to call from a sub-class'
 * constructor because they rely on shared-pointers to be available for `this`.
 */
class FViewModel
	: public ICastable
	, public TSharedFromThis<FViewModel>
	, public FDynamicExtensionContainer
{
public:

	/** Builds a new data model */
	UE_API FViewModel();

	FViewModel(const FViewModel&) = delete;
	FViewModel& operator=(const FViewModel&) = delete;

	UE_API virtual ~FViewModel();

public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FViewModel, FDynamicExtensionContainer);

	/** Adds a dynamic extension to this data model */
	template<typename T, typename... InArgTypes>
	T& AddDynamicExtension(InArgTypes&&... Args)
	{
		T* StaticImpl = this->CastThis<T>();
		if (StaticImpl)
		{
			return *StaticImpl;
		}

		// Assign the dynamic type member so that we can cast directly to these types
		this->DynamicTypes = this;
		return FDynamicExtensionContainer::AddDynamicExtension<T>(AsShared(), Forward<InArgTypes>(Args)...);;
	}

	/** Adds a dynamic extension to this data model */
	template<typename T, typename... InArgTypes>
	T& AddDynamicExtension(const TViewModelTypeID<T>& TypeID, InArgTypes&&... Args)
	{
		return AddDynamicExtension<T>(Forward<InArgTypes>(Args)...);
	}

	/** Casts this data model to an extension, or to a child class implementation */
	template<typename T>
	TViewModelPtr<T> CastThisShared()
	{
		T* Casted = CastThis<T>();
		return TViewModelPtr<T>(AsShared(), Casted ? Casted : CastDynamic<T>());
	}

	/** Casts this data model to an extension, or to a child class implementation */
	template<typename T>
	TViewModelPtr<const T> CastThisShared() const
	{
		const T* Casted = CastThis<const T>();
		return TViewModelPtr<const T>(AsShared(), Casted ? Casted : CastDynamic<const T>());
	}

	/** Casts this data model to an extension, or to a child class implementation */
	template<typename T>
	TSharedPtr<T> CastThisSharedChecked()
	{
		T* Casted = CastThisChecked<T>();
		return TSharedPtr<T>(AsShared(), Casted ? Casted : CastDynamicChecked<T>());
	}

	/** Casts this data model to an extension, or to a child class implementation */
	template<typename T>
	TSharedPtr<const T> CastThisSharedChecked() const
	{
		const T* Casted = CastThisChecked<const T>();
		return TSharedPtr<const T>(AsShared(), Casted ? Casted : CastDynamic<const T>());
	}

	/**
	 * Gets the hierarchical depth of this data model, i.e. the number of levels to
	 * the top-most parent.
	 */
	int32 GetHierarchicalDepth() const
	{
		int32 Depth = 0;
		TSharedPtr<FViewModel> Parent = WeakParent.Pin();
		while (Parent)
		{
			++Depth;
			Parent = Parent->WeakParent.Pin();
		}
		return Depth;
	}

	/** Gets the parent data model */
	UE_API FViewModelPtr GetParent() const;

	/** Gets the parent data model */
	template<typename T>
	TViewModelPtr<T> CastParent() const
	{
		TSharedPtr<FViewModel> Parent = WeakParent.Pin();
		return Parent ? TViewModelPtr<T>(Parent, Parent->CastThis<T>()) : TViewModelPtr<T>();
	}

	/** Returns the root model of the hierarchy, i.e. the parent model that has no parent */
	UE_API FViewModelPtr GetRoot() const;


	/** Get this model's unique, non-persistent and non-deterministic ID */
	uint32 GetModelID() const
	{
		return ModelID;
	}

public:

	UE_API bool IsConstructed() const;

	UE_API TSharedPtr<FSharedViewModelData> GetSharedData() const;
	UE_API void SetSharedData(TSharedPtr<FSharedViewModelData> InSharedData);

	/** Retrieve the first child list that matches the specified type (if any) */
	UE_API TOptional<FViewModelChildren> FindChildList(EViewModelListType InType);
	/** Retrieve the first child list that matches the specified type. Asserts on failure. */
	UE_API FViewModelChildren GetChildList(EViewModelListType InType);

	/** Returns whether this model has any children in any of its child lists */
	UE_API bool HasChildren() const;

	/** Gets the next sibling of this data model in the children list it belongs to */
	UE_API TSharedPtr<FViewModel> GetPreviousSibling() const;

	/** Gets the next sibling of this data model in the children list it belongs to */
	UE_API TSharedPtr<FViewModel> GetNextSibling() const;

public:

	/** Remove this model from its parent */
	UE_API void RemoveFromParent();

	/** Unlinks the head of the children list */
	UE_API void DiscardAllChildren();

public:

	/** Gets an iterator for the data model's default children list */
	UE_API FViewModelListIterator GetChildren(EViewModelListType InFilter = EViewModelListType::Everything) const;

	/** Gets children of a given type from the view-model's children lists, or the specified list */
	template<typename T>
	TViewModelListIterator<T> GetChildrenOfType(EViewModelListType InFilter = EViewModelListType::Everything) const
	{
		return TViewModelListIterator<T>(FirstChildListHead, InFilter);
	}

	/**
	 * Gets all children and their descendants in the hierarchy below this data model.
	 */
	UE_API FParentFirstChildIterator GetDescendants(bool bIncludeThis = false, EViewModelListType InFilter = EViewModelListType::Everything) const;

	/**
	 * Gets all children and descendants of a given type in the hierarchy below this data model.
	 */
	UE_API void GetDescendantsOfType(FViewModelTypeID Type, TArray<TSharedPtr<FViewModel>>& OutChildren, EViewModelListType InFilter = EViewModelListType::Everything) const;

	/**
	 * Gets all children and descendants of a given type in the hierarchy below this data model.
	 */
	template<typename T>
	TParentFirstChildIterator<T> GetDescendantsOfType(bool bIncludeThis = false, EViewModelListType InFilter = EViewModelListType::Everything) const
	{
		return TParentFirstChildIterator<T>(const_cast<FViewModel*>(this)->AsShared(), bIncludeThis, InFilter);
	}

	/**
	 * Finds the first ancestor view model that implements the given type.
	 */
	template<typename T>
	TViewModelPtr<T> FindAncestorOfType(bool bIncludeThis = false) const
	{
		if (auto AncestorsIt = GetAncestorsOfType<T>(bIncludeThis); AncestorsIt)
		{
			return *AncestorsIt;
		}
		return nullptr;
	}

	/**
	 * Finds a parent data model of a given type among the ancestry of this data model.
	 */
	UE_API FViewModelPtr FindAncestorOfType(FViewModelTypeID Type, bool bIncludeThis = false) const;

	/**
	 * Finds a parent data model that supports all the given interface types among the ancestry
	 * of this data model.
	 */
	UE_API FViewModelPtr FindAncestorOfTypes(TArrayView<const FViewModelTypeID> Types, bool bIncludeThis = false) const;

	/**
	 * Iterates over the ancestry of this data model.
	 */
	UE_API FParentModelIterator GetAncestors(bool bIncludeThis = false) const;

	/**
	 * Iterates over all ancestors of a given time along the ancestry of this data model.
	 */
	template<typename T>
	TParentModelIterator<T> GetAncestorsOfType(bool bIncludeThis = false) const
	{
		return TParentModelIterator<T>(SharedThis(const_cast<FViewModel*>(this)), bIncludeThis);
	}

	/**
	 * Returns true if this view model is an ancestor of the given view model.
	 * @param InDescendant The potential descendant to check
	 * @param bIncludeThis If true, returns true when InDescendant.Get() == this
	 */
	bool IsAncestorOf(const TSharedPtr<FViewModel>& InDescendant, bool bIncludeThis = false) const
	{
		TSharedPtr<FViewModel> Current = bIncludeThis ? InDescendant : (InDescendant ? InDescendant->WeakParent.Pin() : nullptr);
		while (Current)
		{
			if (Current.Get() == this)
			{
				return true;
			}
			Current = Current->WeakParent.Pin();
		}
		return false;
	}

protected:

	/** Chain a new children list to the existing children lists */
	UE_API void RegisterChildList(FViewModelListHead* InChildren);

	/** Retrieve the children for the specified child list head */
	UE_API FViewModelChildren GetChildrenForList(FViewModelListHead* ListHead);

	UE_API void SetSharedDataFromConstructor(TSharedPtr<FSharedViewModelData> InSharedData);

private:

	enum class ESetParentResult
	{
		AlreadySameParent,
		ChangedParent,
		ClearedParent,
	};
	UE_API ESetParentResult SetParentOnly(const TSharedPtr<FViewModel>& NewParent, bool bReportChanges = true);

	/** Called once for this model when it is added to a parent that has valid shared data */
	virtual void OnConstruct() {}
	virtual void OnDestruct() {}

	friend FViewModelChildren;
	friend FViewModelListHead;
	friend FScopedViewModelListHead;
	friend FViewModelListLink;
	friend FViewModelListIterator;
	friend FParentFirstChildIterator;
	friend FViewModelHierarchyOperation;
	friend FViewModelIterationState;

	/** Parent of this data model */
	TWeakPtr<FViewModel> WeakParent;
	/** Data that is shared between all view-models in this hierarchy */
	TSharedPtr<FSharedViewModelData> SharedData;
	/** Link to siblings of this data model */
	FViewModelListLink Link;
	/** List of children data models */
	FViewModelListHead* FirstChildListHead;

	/**
	 * Counter that is non-zero when this model is actively being iterated
	 *       that enables detection of invalid move operations during iteration
	 *
	 * Note: this is not set when the model is only a _prospective candidate_
	 *       for iteration (ie a sibling of an item that an iterator is currently
	         looking at). Such models are safe to move due to children being linked lists
	 */
	int32 ActiveIterationCount;

	/** Non-deterministic, non-persistent, serially increasing model ID unique to this instance */
	uint32 ModelID;

	/**
	 * Flag to track whether this view model needs to be constructed
	 */
	uint8 bNeedsConstruction : 1;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
