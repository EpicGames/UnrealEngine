// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "MVVM/SharedList.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/TVariant.h"
#include "SequencerCoreFwd.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCERCORE_API

namespace UE
{
namespace Sequencer
{
class FSharedViewModelData;
class FViewModel;
struct FViewModelChildren;
struct FViewModelListHead;
struct FViewModelSubListIterator;

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	#define UE_SEQUENCER_DETECT_LINK_LIST_CYCLES 1
#else
	#define UE_SEQUENCER_DETECT_LINK_LIST_CYCLES 0
#endif

/**
 * Model list for defining logical groupings of children within a view model
 * Custom list types can be registered using RegisterCustomModelListType
 */
enum class EViewModelListType : uint32
{
	// Generic entry for unidentified or miscellaneous view models
	Generic         = 1u << 0,
	// Entry containing all outliner view models
	Outliner        = 1u << 1,
	// Entry containing all track area view models
	TrackArea       = 1u << 2,
	// Custom entry start
	Custom          = 1u << 4,

	// Recycled models - not included in any iteration by default
	Recycled        = 1u << 30,

	// Symbolic last and invalid entries - should only be used as an error condition
	Last            = 1u << 31,
	Invalid         = Last,

	Everything = (~0u) & ~(Recycled|Invalid),
	Any = Everything,
};
ENUM_CLASS_FLAGS(EViewModelListType)

/**
 * Register a new custom view model list type
 */
SEQUENCERCORE_API EViewModelListType RegisterCustomModelListType();

/**
 * Intrusive linked-list of data models
 *
 *  W A R N I N G : DO NOT USE DIRECTLY UNLESS YOU KNOW WHAT YOU'RE DOING!
 *	This structure won't correctly update the data models' parent pointers,
 *	which can lead to problems. Use the methods on FViewModel.
 *	This link structure is mostly private anyway.
 */
struct FViewModelListLink
{
	UE_API ~FViewModelListLink();

	/** Detach only this link from its list */
	UE_API void Unlink();

	/** Find ths last valid link in the list */
	UE_API TSharedPtr<FViewModel> FindLastLink() const;

private:

	friend FViewModel;
	friend FViewModelListHead;
	friend FViewModelChildren;

	/** Throws an assert if a cycle is detected */
	UE_API void DetectLinkListCycle();

	/** Throws an assert if a cycle is detected */
	static UE_API void DetectLinkListCycle(TSharedPtr<FViewModel> StartAt);

	/** Unlinks the model from its siblings and insert it after the given chain link */
	static UE_API void LinkModelTo(TSharedPtr<FViewModel> Model, TSharedPtr<FViewModelListLink> ToLink);

	/** Next data model in the list */
	TSharedPtr<FViewModel> Next;
	/** Previous item in the list */
	TWeakPtr<FViewModelListLink> WeakPrev;
};

/**
 * Defines the head of a linked list of view models.
 *     This list can actually be considered a linked list of linked-lists
 *     due to each FViewModelListHead object containing a NextList
 *     pointer which allows the segregation of logical groupings of view models.
 *
 *     One example of this would be separating Outliner view models from
 *     track area view models. Keeping these as separate lists allows efficient
 *     filtering of these logical groupings during iteration.
 *
 *  W A R N I N G : DO NOT USE DIRECTLY UNLESS YOU KNOW WHAT YOU'RE DOING!
 *	This structure won't correctly update the data models' parent pointers,
 *	which can lead to problems. Use the methods on FViewModel.
 *	This link structure is mostly private anyway.
 */
struct FViewModelListHead
{
	FViewModelListHead(EViewModelListType InType)
		: Type(InType)
	{}

	/** Returns the head model of this list */
	UE_API FViewModelPtr GetHead() const;

	/** Finds the last model of this list */
	UE_API FViewModelPtr GetTail() const;

	/** Iterate the children held within this sub list */
	UE_API FViewModelSubListIterator Iterate() const;

	/** Iterate the children held within this sub list */
	template<typename T>
	TViewModelSubListIterator<T> Iterate() const
	{
		return TViewModelSubListIterator<T>(this);
	}

	/**
	 * Move all the children in this list to a new list.
	 * @note: does not change parent relationships - NewHead must be within the same parent
	 */
	UE_API TSharedPtr<FViewModel> ReliquishList();

	/** Head of this list, where Next is the first item */
	FViewModelListLink HeadLink;
	/** Next list, for when data models have multiple lists of children */
	FViewModelListHead* NextListHead = nullptr;
	/** Identifier for this list head */
	EViewModelListType Type;
};

/**
 * Scoped object that adds a temporary child list to the specified model of the specified type.
 * Primarily used during regeneration of children to temporarily introduce a 'Recycled' child list
 */
struct FScopedViewModelListHead
{
	/** Constructor that adds the list to the model */
	UE_API FScopedViewModelListHead(const TSharedPtr<FViewModel>& InModel, EViewModelListType InType);
	/** Destructor that removes the list to the model */
	UE_API ~FScopedViewModelListHead();

	/** Non-copyable and non-moveable since a pointer to our ListHead member is added to our owner */
	FScopedViewModelListHead(const FScopedViewModelListHead&) = delete;
	FScopedViewModelListHead& operator=(const FScopedViewModelListHead&) = delete;

	FScopedViewModelListHead(FScopedViewModelListHead&&) = delete;
	FScopedViewModelListHead& operator=(FScopedViewModelListHead&&) = delete;

	/**
	 * Retrieve this child list
	 */
	UE_API FViewModelChildren GetChildren();

private:

	/** The model that contains our child list */
	TSharedPtr<FViewModel> Model;
	/** List head whose pointer is added to Model's ChildLists */
	FViewModelListHead ListHead;
};

struct FViewModelChildren
{
	/**
	 * Get the collective type of these children
	 */
	UE_API EViewModelListType GetType() const;

	/**
	 * Get this children's parent
	 */
	UE_API FViewModelPtr GetParent() const;

	/**
	 * Add a child from a different parent to these children
	 * @note: If the child is already under this parent, this function is a noop
	 */
	UE_API void AddChild(const TSharedPtr<FViewModel>& Child);

	/**
	 * Add a child from a different parent to these children to the specified previous sibling, or the head if nullptr
	 * @note: If the child is already the specified model's next sibling, this function is a noop
	 */
	UE_API void InsertChild(const TSharedPtr<FViewModel>& Child, const TSharedPtr<FViewModel>& PreviousSibling);

	/**
	 * Move all these children into the specified destination list, changing parent pointers if necessary
	 */
	UE_API void MoveChildrenTo(const FViewModelChildren& OutDestination);

	/**
	 * Invoke a callback on these children, and move them all into the specified destination list, changing parent pointers if necessary
	 */
	template<typename T, typename Callback>
	void MoveChildrenTo(const FViewModelChildren& OutDestination, Callback&& InCallback)
	{
		for (const TViewModelPtr<T>& Item : IterateSubList<T>())
		{
			InCallback(Item);
		}
		MoveChildrenTo(OutDestination);
	}

	/**
	 * Empty this specific sub-list, resetting all parent ptrs
	 */
	UE_API void Empty();

public:

	/** Returns an iterator over this list */
	UE_API FViewModelSubListIterator IterateSubList() const;

	/** Returns an iterator over this list which will skip any item not of type T */
	template<typename T>
	TViewModelSubListIterator<T> IterateSubList() const
	{
		return TViewModelSubListIterator<T>(ListHead);
	}

	/** Returns whether this list is empty */
	bool IsEmpty() const
	{
		return !ListHead->HeadLink.Next.IsValid();
	}

	/** Returns the head model of this list */
	UE_API FViewModelPtr GetHead() const;

	/** Finds the last model of this list */
	UE_API FViewModelPtr GetTail() const;

	template<typename T>
	TViewModelPtr<T> FindFirstChildOfType() const
	{
		for (const TViewModelPtr<T>& Child : IterateSubList<T>())
		{
			return Child;
		}
		return nullptr;
	}

private:

	friend FViewModel;
	friend FScopedViewModelListHead;

	UE_API FViewModelChildren(const TSharedPtr<FViewModel>& InOwner, FViewModelListHead* InListHead);

	TSharedPtr<FViewModel> Owner;
	FViewModelListHead* ListHead;
};


/**
 * Scoped utility class for batching change notifications on a view model hierarchy
 * This class works by keeping track of models that have been changed in some way,
 * along with their previous sibling and parent pointers before any change was made.
 * 
 * With this information, when all FViewModelHierarchyOperation's have gone out of scope
 * we can compare everything with its original pointers, and only dispatch change events
 * in the case where they have actually changed. This enables reliable and accurate event
 * processing in cases where large updates are made to a hierarchy with little or no
 * meaningful change to the hierarchy.
 * 
 * Within any given hierarhcy of FViewModelHierarchyOperation scopes, there is guaranteed to
 * be a maximum of one event triggered for each changed node. This is achieved by tracking
 * which models have already been handled during the event processing.
 */
class FViewModelHierarchyOperation
{
public:

	template<typename ViewModelType>
	UE_DEPRECATED(5.5, "Please use the FSharedViewModelData overload by passing ->GetSharedData()")
	explicit FViewModelHierarchyOperation(const TSharedPtr<ViewModelType>& InAnyModel)
		: SharedData(InAnyModel->SharedData)
	{
		Construct();
	}

	UE_API explicit FViewModelHierarchyOperation(const TSharedPtr<FSharedViewModelData>& InSharedData);
	UE_API ~FViewModelHierarchyOperation();

	/**
	 * Called before a change is made to the specified model's parent or previous sibling
	 */
	UE_API void PreHierarchicalChange(const TSharedPtr<FViewModel>& InChangedNode);

private:

	UE_API void Construct();

	struct FCachedHierarchicalPosition
	{
		TWeakPtr<FViewModel> PreviousParent;
		TWeakPtr<FViewModel> PreviousSibling;
	};
	struct FOperationAccumulationBuffer
	{
		TMap<TWeakPtr<FViewModel>, FCachedHierarchicalPosition> ChangedModels;
	};

	/** Shared data that contains the bound event handlers for any model */
	TSharedPtr<FSharedViewModelData> SharedData;
	/** Accumulation buffer shared between all active FViewModelHierarchyOperations */
	TSharedPtr<FOperationAccumulationBuffer> AccumulationBuffer;
	/** Pointer to the most recently active operation before this one existed */
	FViewModelHierarchyOperation* OldOperation;
};



} // namespace Sequencer
} // namespace UE

#undef UE_API
