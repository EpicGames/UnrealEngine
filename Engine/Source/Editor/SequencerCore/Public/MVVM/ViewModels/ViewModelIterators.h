// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Invoke.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

namespace UE
{
namespace Sequencer
{
template <typename T, typename IteratorType> struct TTypedIterator;

/**
 * Iterator state that wraps a view model and handles incrementing/decrementing
 * its ActiveIterationCount while active
 */
struct FViewModelIterationState
{
	FViewModelPtr ViewModel;

	FViewModelIterationState() = default;
	SEQUENCERCORE_API FViewModelIterationState(FViewModelPtr&& InViewModel);

	SEQUENCERCORE_API FViewModelIterationState(const FViewModelIterationState&);
	SEQUENCERCORE_API FViewModelIterationState& operator=(const FViewModelIterationState&);

	SEQUENCERCORE_API FViewModelIterationState(FViewModelIterationState&&);
	SEQUENCERCORE_API FViewModelIterationState& operator=(FViewModelIterationState&&);

	SEQUENCERCORE_API ~FViewModelIterationState();
};



// WARNING: most of the class and methods in this header are not safe to use from a view-model's
// constructor because they rely on shared-pointers to be available for that view-model.

template<typename DerivedType, typename EnumeratedType>
struct TPopulatableIteratorMixin
{

	/**
	 * Populate the specified container with the remaining items in this iterator.
	 * ContainerType must have a .Add method that accepts the enumerated type.
	 * The state of this iterator remains unchanged.
	 * 
	 * Example usage:
	 * 
	 * TSet<FViewModelPtr> Set;
	 * It.Populate(Set);
	 */
	template<typename ContainerType>
	void Populate(ContainerType& OutContainer) const
	{
		for (const EnumeratedType& Item : static_cast<const DerivedType&>(*this))
		{
			OutContainer.Add(Item);
		}
	}

	/**
	 * Create a new container of the templated type and populate it with the remaining items in this iterator.
	 * ContainerType must have a .Add method that accepts the enumerated type.
	 * The state of this iterator remains unchanged.
	 * 
	 * Example usage:
	 * 
	 * TSet<TViewModelPtr<FTrackModel>> Tracks = ViewModel->GetDescendentsOfType<FTrackModel>().Populate<TSet>();
	 */
	template<template<typename T, typename ...Rest> class ContainerType>
	ContainerType<EnumeratedType> Populate() const
	{
		ContainerType<EnumeratedType> Temp;
		Populate(Temp);
		return Temp;
	}

	/**
	 * Populate the specified array with the remaining items in this iterator
	 * The state of this iterator remains unchanged
	 */
	void ToArray(TArray<EnumeratedType>& OutArray) const
	{
		Populate(OutArray);
	}

	/**
	 * Return the remaining items in this iterator as an array
	 * The state of this iterator remains unchanged
	 */
	TArray<EnumeratedType> ToArray() const
	{
		return Populate<TArray>();
	}

	/**
	 * Linearly search through this iterator to find a predicate using a projection
	 * For example:
	 *      UMovieSceneTrack* TrackToFind = ...;
	 *      Model->GetChildrenOfType<FTrackModel>().FindBy(TrackToFind, &FTrackModel::GetTrack)
	 */
	template<typename ValueType, typename ProjectionType>
	EnumeratedType FindBy(const ValueType& Value, ProjectionType Proj) const
	{
		for (const EnumeratedType& Item : static_cast<const DerivedType&>(*this))
		{
			if (Invoke(Proj, *Item) == Value)
			{
				return Item;
			}
		}
		return nullptr;
	}
};

/**
 * Iterates all models that exist in a single list from the start model
 */
struct FViewModelSubListIterator : TPopulatableIteratorMixin<FViewModelSubListIterator, FViewModelPtr>
{
	FViewModelSubListIterator() = default;

	explicit FViewModelSubListIterator(const FViewModelListHead* List)
		: State(List ? List->GetHead() : nullptr)
	{}

	explicit FViewModelSubListIterator(TSharedPtr<FViewModel> InStart)
		: State(InStart)
	{}

	/** Returns whether iterators point to the same data models */
	FORCEINLINE friend bool operator!=(const FViewModelSubListIterator& A, const FViewModelSubListIterator& B) { return A.State.ViewModel != B.State.ViewModel; }

	/** Returns whether the iterator is pointing to a valid data model */
	FORCEINLINE explicit operator bool() const { return (bool)State.ViewModel; }

	/** Dereferences the iterator to the pointed-to data model */
	FORCEINLINE FViewModel* operator->() const { check(State.ViewModel); return State.ViewModel.Get(); }
	/** Dereferences the iterator to the pointed-to data model */
	FORCEINLINE const FViewModelPtr& operator*() const { return State.ViewModel; }

	/** Moves to the next sibling of the pointed-to data model */
	FORCEINLINE void operator++() { check(State.ViewModel); State = FViewModelIterationState(State.ViewModel->GetNextSibling()); }

	/** Gets the pointed-to data model */
	FORCEINLINE FViewModelPtr GetCurrentItem() const { return State.ViewModel; }

	FORCEINLINE FViewModelSubListIterator begin() const { return *this; }
	FORCEINLINE FViewModelSubListIterator end() const   { return FViewModelSubListIterator{ nullptr }; }

private:

	/** The pointed-to data model */
	FViewModelIterationState State;
};

/**
 * Iterates all models contained in the specified list chain
 */
struct FViewModelListIterator : TPopulatableIteratorMixin<FViewModelListIterator, FViewModelPtr>
{
	FViewModelListIterator(EViewModelListType InFilter = EViewModelListType::Everything)
		: NextList(nullptr)
		, Filter(InFilter)
	{}

	explicit FViewModelListIterator(const TSharedPtr<FViewModel>& InModel, EViewModelListType InFilter = EViewModelListType::Everything)
		: NextList(InModel ? InModel->FirstChildListHead : nullptr)
		, Filter(InFilter)
	{
		FindNextValidIterator();
	}

	explicit FViewModelListIterator(const FViewModelListHead* InListHead, EViewModelListType InFilter = EViewModelListType::Everything)
		: NextList(InListHead)
		, Filter(InFilter)
	{
		FindNextValidIterator();
	}

	FORCEINLINE friend bool operator!=(const FViewModelListIterator& A, const FViewModelListIterator& B)
	{
		return A.ListIterator != B.ListIterator || A.NextList != B.NextList;
	}
	FORCEINLINE explicit              operator bool()  const { return (bool)ListIterator;  }
	FORCEINLINE FViewModel*   operator->()     const { return ListIterator.operator->(); }
	FORCEINLINE const FViewModelPtr&  operator*()      const { return *ListIterator; }
	FORCEINLINE FViewModelPtr         GetCurrentItem() const { return ListIterator.GetCurrentItem(); }

	void operator++()
	{
		++ListIterator;
		FindNextValidIterator();
	}

	FORCEINLINE FViewModelListIterator begin() const { return *this; }
	FORCEINLINE FViewModelListIterator end() const   { return FViewModelListIterator{ nullptr }; }

private:

	void FindNextValidIterator()
	{
		while (!ListIterator && NextList)
		{
			// Skip lists that do not match the filter
			while (NextList && !EnumHasAnyFlags(NextList->Type, Filter))
			{
				NextList = NextList->NextListHead;
			}

			if (NextList)
			{
				ListIterator = FViewModelSubListIterator(NextList);
				NextList = NextList->NextListHead;
			}
		}
	}

private:

	const FViewModelListHead* NextList;
	FViewModelSubListIterator ListIterator;
	EViewModelListType Filter;
};

/**
 * An iterator that can accomodate different types of data model lists, arrays, etc.
 */
struct FViewModelVariantIterator
{
	FViewModelVariantIterator()
		: Value(nullptr)
		, Data(nullptr)
		, ArrayProjection(nullptr)
		, ArrayIndex(0)
		, ArrayNum(0)
	{
	}

	FViewModelVariantIterator(const FViewModelListHead* DirectList)
		: Value(DirectList->GetHead())
		, Data(DirectList)
		, ArrayProjection(nullptr)
		, ArrayIndex(0)
		, ArrayNum(0)
	{
	}

	template<typename T>
	FViewModelVariantIterator(const TArray<TWeakViewModelPtr<T>>* InWeakArrayView)
		: Data(InWeakArrayView->GetData())
		, ArrayIndex(-1) // Start off the front of the array and operator++ into it
		, ArrayNum(InWeakArrayView->Num())
	{
		ArrayProjection = &ProjectWeak<T>;
		operator++();
	}

	template<typename T>
	FViewModelVariantIterator(const TArray<TViewModelPtr<T>>* InStrongArrayView)
		: Data(InStrongArrayView->GetData())
		, ArrayIndex(-1) // Start off the front of the array and operator++ into it
		, ArrayNum(InStrongArrayView->Num())
	{
		ArrayProjection = &ProjectStrong<T>;
		operator++();
	}

	FORCEINLINE const FViewModelPtr& GetCurrentItem() const
	{
		return Value;
	}
	FORCEINLINE const FViewModelPtr& operator*() const
	{
		return Value;
	}
	FORCEINLINE FViewModel* operator->() const
	{
		check(Value);
		return Value.Get();
	}
	FORCEINLINE explicit operator bool() const
	{
		return (bool)Value;
	}
	FORCEINLINE void operator++()
	{
		if (ArrayProjection)
		{
			Value = nullptr;
			while (!Value && ArrayIndex < ArrayNum-1)
			{
				++ArrayIndex;
				Value = ArrayProjection(ArrayIndex, Data);
			}
		}
		else
		{
			Value = Value->GetNextSibling();
		}
	}
	FORCEINLINE friend bool operator!=(const FViewModelVariantIterator& A, const FViewModelVariantIterator& B)
	{
		return A.Value != B.Value || A.Data != B.Data;
	}

	FORCEINLINE FViewModelVariantIterator begin() const { return *this; }
	FORCEINLINE FViewModelVariantIterator end() const   { FViewModelVariantIterator List; List.Data = Data; return List; }

private:

	template<typename T>
	static TSharedPtr<FViewModel> ProjectWeak(int32 Index, const void* InData)
	{
		const TWeakPtr<T>* DataPtr = static_cast<const TWeakPtr<T>*>(InData);
		return DataPtr[Index].Pin();
	}
	template<typename T>
	static TSharedPtr<FViewModel> ProjectStrong(int32 Index, const void* InData)
	{
		const TSharedPtr<T>* DataPtr = static_cast<const TSharedPtr<T>*>(InData);
		return DataPtr[Index];
	}

	using ProjectionType = TSharedPtr<FViewModel> (*)(int32 Index, const void*);

	FViewModelPtr Value;
	const void* Data;
	ProjectionType ArrayProjection;
	int32 ArrayIndex;
	int32 ArrayNum;
};

/**
 * A depth-first iterator into a hierarchy of data models. It iterates over *all* registered children lists of each model.
 */
struct FParentFirstChildIterator
{
	FParentFirstChildIterator(EViewModelListType InFilter = EViewModelListType::Everything)
		: Filter(InFilter)
		, DepthLimit(-1)
		, bIgnoreCurrentChildren(false)
	{}

	SEQUENCERCORE_API FParentFirstChildIterator(const TSharedPtr<FViewModel>& StartAt, bool bIncludeThis = false, EViewModelListType InFilter = EViewModelListType::Everything);

	FORCEINLINE int32 GetCurrentDepth() const
	{
		return IterStack.Num();
	}
	FORCEINLINE const FViewModelPtr& GetCurrentItem() const
	{
		return CurrentItem;
	}
	FORCEINLINE const FViewModelPtr& operator*() const
	{
		return CurrentItem;
	}
	FORCEINLINE FViewModel* operator->() const
	{
		return CurrentItem.Get();
	}
	FORCEINLINE explicit operator bool() const
	{
		return (bool)CurrentItem;
	}
	FORCEINLINE void operator++()
	{
		if (bIgnoreCurrentChildren)
		{
			// reset the flag before we iterate!
			bIgnoreCurrentChildren = false;
			IterateToNextSibling();
		}
		else
		{
			IterateToNext();
		}
	}
	FORCEINLINE void SetMaxDepth()
	{
		DepthLimit = IterStack.Num();
	}
	FORCEINLINE void SetMaxDepth(int32 InDepthLimit)
	{
		DepthLimit = InDepthLimit;
	}
	FORCEINLINE friend bool operator!=(const FParentFirstChildIterator& A, const FParentFirstChildIterator& B)
	{
		return A.CurrentItem != B.CurrentItem;
	}

	/**
	 * Request that this iterator ignore any and all children of the current model
	 * @note: Does not change the state of this iterator until it is incremented
	 */
	FORCEINLINE void IgnoreCurrentChildren()
	{
		bIgnoreCurrentChildren = true;
	}

	FORCEINLINE FParentFirstChildIterator begin() const { return *this; }
	FORCEINLINE FParentFirstChildIterator end() const   { return FParentFirstChildIterator(); }

protected:

	SEQUENCERCORE_API void IterateToNext();

	SEQUENCERCORE_API bool IterateToNextChild();

	SEQUENCERCORE_API void IterateToNextSibling();

protected:

	/** The model currently pointed to by this iterator */
	FViewModelPtr CurrentItem;

	/** Stack of parent's that have been visited in the current branch */
	TArray<FViewModelListIterator, TInlineAllocator<8>> IterStack;

	/** List type filter to allow skipping of different child lists */
	EViewModelListType Filter;

	/** Maximum depth that this iterator is allowed to recurse to */
	int32 DepthLimit;

	/** Flag that signals this iterator to skip all children of the current item and iterate directly to its sibling */
	bool bIgnoreCurrentChildren;
};

/** Iterator going up the parent relationships towards the root */
struct FParentModelIterator
{
	FParentModelIterator() = default;

	FParentModelIterator(TSharedPtr<FViewModel> StartAtChild, bool bIncludeThis = false)
	{
		if (bIncludeThis)
		{
			Node = StartAtChild;
		}
		else if (StartAtChild)
		{
			Node = StartAtChild->GetParent();
		}
	}

	FORCEINLINE const FViewModelPtr& GetCurrentItem() const
	{
		return Node;
	}
	FORCEINLINE const FViewModelPtr& operator*() const
	{
		return Node;
	}
	FORCEINLINE FViewModel* operator->() const
	{
		return Node.Get();
	}
	FORCEINLINE explicit operator bool() const
	{
		return (bool)Node;
	}
	FORCEINLINE void operator++()
	{
		Node = Node->GetParent();
	}
	FORCEINLINE friend bool operator!=(const FParentModelIterator& A, const FParentModelIterator& B)
	{
		return A.Node != B.Node;
	}

	FORCEINLINE FParentModelIterator begin() const { return *this; }
	FORCEINLINE FParentModelIterator end() const   { return FParentModelIterator(); }

protected:

	FViewModelPtr Node;
};

/**
 * Utility class to wrap an untyped iterator (one that only returns FViewModel pointers)
 * so that it can be used as a typed iterator.
 */
template<typename T, typename IteratorType>
struct TTypedIteratorBase : public IteratorType, public TPopulatableIteratorMixin<TTypedIteratorBase<T, IteratorType>, TViewModelPtr<T>>
{
	template<typename ...ArgTypes>
	TTypedIteratorBase(ArgTypes&&... InArgs)
		: IteratorType(Forward<ArgTypes>(InArgs)...)
	{
		GetCastResultOrContinue();
	}

	FORCEINLINE T*                               operator->()     const { check(TypedValue); return TypedValue.operator->(); }
	FORCEINLINE const TViewModelPtr<T>&          operator*()      const { check(TypedValue); return TypedValue;       }

	void operator++()
	{
		IteratorType::operator++();
		GetCastResultOrContinue();
	}

	FORCEINLINE TTypedIterator<T, IteratorType> begin() const { return IteratorType::begin(); }
	FORCEINLINE TTypedIterator<T, IteratorType> end() const   { return IteratorType::end(); }

	using TPopulatableIteratorMixin<TTypedIteratorBase<T, IteratorType>, TViewModelPtr<T>>::ToArray;
	using TPopulatableIteratorMixin<TTypedIteratorBase<T, IteratorType>, TViewModelPtr<T>>::Populate;
	using TPopulatableIteratorMixin<TTypedIteratorBase<T, IteratorType>, TViewModelPtr<T>>::FindBy;

protected:

	void GetCastResultOrContinue()
	{
		while (*this)
		{
			TypedValue = IteratorType::GetCurrentItem()->template CastThisShared<T>();
			if (TypedValue)
			{
				break;
			}

			// operator++ must not be called if the break; clause is hit above
			IteratorType::operator++();
		}
	}

	TViewModelPtr<T> TypedValue;
};

template<typename T, typename IteratorType>
struct TTypedIterator : TTypedIteratorBase<T, IteratorType>
{
	template<typename ...ArgTypes>
	TTypedIterator(ArgTypes&&... InArgs)
		: TTypedIteratorBase<T, IteratorType>(Forward<ArgTypes>(InArgs)...)
	{}
};

template<typename T>
struct TTypedIterator<T, FParentFirstChildIterator> : TTypedIteratorBase<T, FParentFirstChildIterator>
{
	template<typename ...ArgTypes>
	TTypedIterator(ArgTypes&&... InArgs)
		: TTypedIteratorBase<T, FParentFirstChildIterator>(Forward<ArgTypes>(InArgs)...)
	{}
};

template<typename T>
using TParentModelIterator = TTypedIterator<T, FParentModelIterator>;
template<typename T>
using TParentFirstChildIterator = TTypedIterator<T, FParentFirstChildIterator>;
template<typename T>
using TViewModelListIterator = TTypedIterator<T, FViewModelListIterator>;
template<typename T>
using TViewModelSubListIterator = TTypedIterator<T, FViewModelSubListIterator>;

} // namespace Sequencer
} // namespace UE

