// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Layout/Geometry.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "TimeToPixel.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer { class FViewModel; }

namespace UE
{
namespace Sequencer
{

class FTrackAreaViewModel;
class SOutlinerView;

/** Structure used for handling the virtual space of the track area */
class FVirtualTrackArea : public FTimeToPixel
{
public:

	/** Construction responsibility is delegated to SSequencer. See SSequencer::GetVirtualTrackArea */
	UE_API FVirtualTrackArea(const FTrackAreaViewModel& InTrackArea, SOutlinerView& InTreeView, const FGeometry& InTrackAreaGeometry);

	/** Convert the specified pixel position into a virtual vertical offset from the absolute top of the tree */
	UE_API float PixelToVerticalOffset(float InPixel) const;

	/** Convert the specified absolute vertical position into a physical vertical offset in the track area. */
	/** @note: Use with caution - not reliable where the specified offset is not on screen */
	UE_API float VerticalOffsetToPixel(float InOffset) const;

	/** Convert the specified physical point into a virtual point the absolute top of the tree */
	UE_API FVector2D PhysicalToVirtual(FVector2D InPosition) const;

	/** Convert the specified absolute virtual point into a physical point in the track area. */
	/** @note: Use with caution - not reliable where the specified point is not on screen */
	UE_API FVector2D VirtualToPhysical(FVector2D InPosition) const;

	/** Get the physical size of the track area */
	UE_API FVector2D GetPhysicalSize() const;

	/** Hit test at the specified physical position for a sequencer node */
	UE_API TSharedPtr<FViewModel> HitTestNode(float InPhysicalPosition) const;

	/** Cached track area geometry */
	FGeometry CachedTrackAreaGeometry() const { return TrackAreaGeometry; }

private:

	/** Reference to the sequencer tree */
	SOutlinerView& TreeView;

	/** Cached physical geometry of the track area */
	FGeometry TrackAreaGeometry;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
