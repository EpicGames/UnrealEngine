// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STreeView.h"
#include "MVVM/ViewModels/ViewModel.h"

namespace UE
{
namespace Sequencer
{

class ISequencerTreeView : public STreeView<TWeakPtr<FViewModel>>
{
};

} // namespace Sequencer
} // namespace UE
