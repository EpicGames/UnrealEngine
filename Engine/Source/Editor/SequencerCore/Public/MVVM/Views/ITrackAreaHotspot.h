// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Styling/SlateBrush.h"
#include "Misc/FrameTime.h"

#include "MVVM/ICastable.h"

#define UE_API SEQUENCERCORE_API

class FExtender;
class FMenuBuilder;
struct FGeometry;
struct FPointerEvent;

namespace UE
{
namespace Sequencer
{

class FTrackAreaViewModel;
class ISequencerEditToolDragOperation;

/** A sequencer hotspot is used to identify specific areas on the sequencer track area */ 
struct ITrackAreaHotspot 
	: public ICastable
{
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, ITrackAreaHotspot);

	virtual ~ITrackAreaHotspot() = default;
	virtual void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const = 0;
	virtual TOptional<FFrameNumber> GetTime() const { return TOptional<FFrameNumber>(); }
	virtual TOptional<FFrameTime> GetOffsetTime() const { return TOptional<FFrameTime>(); }
	virtual TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent) { return nullptr; }
	virtual bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime){ return false; }
	virtual FCursorReply GetCursor() const { return FCursorReply::Unhandled(); }
	virtual const FSlateBrush* GetCursorDecorator(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const { return nullptr; }

	/** Priority when multiple hotspots are present - highest wins */
	virtual int32 Priority() const { return 0; }
};

template<typename T>
FORCEINLINE TSharedPtr<T> HotspotCast(const TSharedPtr<ITrackAreaHotspot>& InHotspot)
{
	if (ITrackAreaHotspot* HotspotPtr = InHotspot.Get())
	{
		T* Result = HotspotPtr->CastThis<T>();
		if (Result)
		{
			return TSharedPtr<T>(InHotspot, Result);
		}
	}
	return nullptr;
}

} // namespace Sequencer
} // namespace UE

#undef UE_API
