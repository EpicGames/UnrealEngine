// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Math/Vector2D.h"

struct FSlateBrush;

enum class EKeyConnectionStyle
{
	None = 0,
	Dotted = 1 << 0,
	Dashed = 1 << 1,
	Solid  = 1 << 2,
};
ENUM_CLASS_FLAGS(EKeyConnectionStyle)

/**
 * Structure defining how a key should be drawn
 */
struct FKeyDrawParams
{
	FKeyDrawParams()
		: BorderBrush(nullptr), FillBrush(nullptr)
		, BorderTint(FLinearColor::White), FillTint(FLinearColor::White)
		, FillOffset(0.f, 0.f)
		, ConnectionStyle(EKeyConnectionStyle::None)
	{}

	friend bool operator==(const FKeyDrawParams& A, const FKeyDrawParams& B)
	{
		return A.BorderBrush == B.BorderBrush && A.FillBrush == B.FillBrush && A.BorderTint == B.BorderTint && A.FillTint == B.FillTint && A.FillOffset == B.FillOffset && A.ConnectionStyle == B.ConnectionStyle;
	}
	friend bool operator!=(const FKeyDrawParams& A, const FKeyDrawParams& B)
	{
		return !(A == B);
	}

	/** Brush to use for drawing the key's border */
	const FSlateBrush* BorderBrush;

	/** Brush to use for drawing the key's filled area */
	const FSlateBrush* FillBrush;

	/** Tint to be used for the key's border */
	FLinearColor BorderTint;

	/** Tint to be used for the key's filled area */
	FLinearColor FillTint;

	/** The amount to offset the fill brush from the keys center */
	FVector2D FillOffset;

	/** Flags denoting how to draw connecting lines between this key and the next */
	EKeyConnectionStyle ConnectionStyle;
};
