// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Math/Range.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "MVVM/Views/KeyDrawParams.h"
#include "MVVM/Extensions/IKeyExtension.h"
#include "Rendering/RenderingCommon.h"
#include "Curves/KeyHandle.h"
#include "TimeToPixel.h"
#include "MVVM/ViewModelPtr.h"

#define UE_API SEQUENCERCORE_API

struct FSlateBrush;
class FWidgetStyle;
class FSlateWindowElementList;

namespace UE::Sequencer
{

/** Key selection preview state - mirrored from FSequencer while selection concepts are ported to SequencerCore */
enum class EKeySelectionPreviewState : uint8
{
	Undefined,
	Selected,
	NotSelected
};

/** The style to draw a key */
enum class EKeyRendererStyle : uint8
{
	Circle,
	Diamond,
	Line,
	FrameBlock
};

/**
 * Paint arguments required for painting keys on a sequencer track
 */
struct FKeyRendererPaintArgs
{
	/** Draw elements to paint onto */
	FSlateWindowElementList* DrawElements;

	/** The amount to throb selected keys by */
	float KeyThrobValue = 0.f;

	/** The amount to throb selected sections by */
	float SectionThrobValue = 0.f;

	/** Fixed amount to throb newly created keys by */
	FVector2D ThrobAmount = FVector2D(12.f, 12.f);

	/** Default draw effect */
	ESlateDrawEffect DrawEffects;

	/** Curve color to paint underlying curves with */
	FLinearColor CurveColor = FLinearColor(0.8f, 0.8f, 0.f, 0.7f);

	/** Key bar color */
	FLinearColor KeyBarColor = FColor(160, 160, 160);

	/** Style used to draw each key */
	EKeyRendererStyle KeyStyle = EKeyRendererStyle::Circle;

	/** Key draw offset */
	FVector2D KeyOffset = FVector2D::ZeroVector;

	/** The fill color to use to display keys on the timeline */
	FLinearColor KeyFillColor = FLinearColor(.83f, .12f, .12f, 1.f);

	/** The fill color to use to display hovered keys on the timeline */
	FLinearColor HoveredKeyFillColor = FLinearColor::White;

	/** If true, hovered keys blend between their base fill color and HoveredKeyFillColor. */
	bool bBlendHoveredKeyColorWithBaseKeyColor = false;

	/** Blend amount used when bBlendHoveredKeyColorWithBaseKeyColor is true. */
	float HoveredKeyBlendFactor = .5f;

	/** The fill color to use to display selected keys on the timeline */
	FLinearColor KeySelectionFillColor = HoveredKeyFillColor;

	/** The fill color to use to display partial keys on the timeline */
	FLinearColor PartialKeyFillColor = FLinearColor::White;

	/** The fill color to use to display selected partial keys on the timeline */
	FLinearColor PartialKeySelectionFillColor = KeySelectionFillColor;
};

/** Flag enum signifying how the cache has changed since it was last generated */
enum class EViewDependentCacheFlags : uint8
{
	None             = 0,       // The cache is still entirely valid, simply redraw the keys
	DataChanged      = 1 << 0,  // The underlying keyframes have changed - everything needs regenerating
	KeyStateChanged  = 1 << 1,  // The selection, hover or preview selection state of the keys has changed
	ViewChanged      = 1 << 2,  // The view range has changed - view dependent data needs regenerating, but some cache data may be preserved
	ViewZoomed       = 1 << 3,  // The view range has been zoomed - view dependent data needs regenerating, no key grouping can be preserved

	Empty            = 1 << 4,  // The view range is empty and there is nothing to show

	All = DataChanged | KeyStateChanged | ViewChanged | ViewZoomed,
};

/** Interface for defining selection states for keys */
class IKeyRendererInterface
{
public:
	virtual ~IKeyRendererInterface()= default;

	virtual bool HasAnySelectedKeys() const = 0;
	virtual bool HasAnyPreviewSelectedKeys() const = 0;
	virtual bool HasAnyHoveredKeys() const = 0;

	virtual bool IsKeySelected(const TViewModelPtr<IKeyExtension>&, FKeyHandle) const = 0;
	virtual bool IsKeyHovered(const TViewModelPtr<IKeyExtension>&, FKeyHandle) const = 0;
	virtual EKeySelectionPreviewState GetPreviewSelectionState(const TViewModelPtr<IKeyExtension>&, FKeyHandle) const = 0;
};

struct FKeyBatchParameters
{
	FKeyBatchParameters(const FTimeToPixel& InTimeToPixel)
		: ClientInterface(nullptr)
		, TimeToPixel(InTimeToPixel)
		, KeySizePx(12.f, 12.f)
		, ValidPlayRangeMin(TNumericLimits<int32>::Min())
		, ValidPlayRangeMax(TNumericLimits<int32>::Max())
		, CacheState(EViewDependentCacheFlags::None)
		, bShowCurve(false)
		, bShowKeyBars(true)
		, bCollapseChildren(false)
		, bUpdateViewIndependentData(true)
	{}

	/** Interface ptr for defining selection states for keys. Can be null if selection is not supported. */
	IKeyRendererInterface* ClientInterface;

	/** Time to pixel convertor */
	FTimeToPixel TimeToPixel;

	/** Visible range to clip rendering to */
	TRange<FFrameTime> VisibleRange;

	/** Screen-space size of keys in px */
	FVector2D KeySizePx;

	/** Valid range for keys - keys outside this range are drawn de-saturated */
	FFrameNumber ValidPlayRangeMin;
	FFrameNumber ValidPlayRangeMax;

	/** Cache state that defines whether and how we need to update any cached view (in)dependent data */
	EViewDependentCacheFlags CacheState;

	/** Whether to show a curve underneath the keys */
	uint8 bShowCurve : 1;

	/** Whether to show key bars connecting each key */
	uint8 bShowKeyBars : 1;

	/** Whether to collapse all children of the model into this key renderer as well, drawing all nested keys as groups within this renderer */
	uint8 bCollapseChildren : 1;

	/** Whether to poll key extensions for view-independent key data changes during this update. */
	uint8 bUpdateViewIndependentData : 1;
};

/**
 * Utility class for efficiently drawing large numbers of keys on a track lane.
 */
struct FKeyRenderer
{
	/** Flag enum signifying states for a particular key or group of keys */
	enum class EKeyRenderingFlags
	{
		None               = 0,
		PartialKey         = 1 << 0,  // Indicates that this key comprises multiple keys of different types, or a partially keyed collapsed channel
		Selected           = 1 << 1,  // Only if.NumSelected == Key.TotalNumKeys
		PreviewSelected    = 1 << 2,  // Only if NumPreviewSelected == NumKeys
		PreviewNotSelected = 1 << 3,  // Only if NumPreviewNotSelected == NumKeys
		AnySelected        = 1 << 4,  // If any are selected
		Hovered            = 1 << 5,  // Only if NumKeys == NumHovered
		Overlaps           = 1 << 6,  // If NumKeys > 1
		OutOfRange         = 1 << 7,  // If any of the keys fall outside of the valid range
		SubFrame           = 1 << 8,  // If the key is on a sub frame or a whole frame
	};

	enum class EKeyBarRenderingFlags
	{
		None               = 0,
		Hovered            = 1 << 0,
		OutOfRange         = 1 << 1,
	};

	struct FKeysForModel
	{
		FViewModelPtr Model;
		TArray<FKeyHandle> Keys;
	};

	struct FKeyBar
	{
		TRange<FFrameTime> Range = TRange<FFrameTime>::Empty();
		TArray<FKeysForModel> LeadingKeys;
		TArray<FKeysForModel> TrailingKeys;

		bool IsValid() const
		{
			return !Range.IsEmpty();
		}
	};

	/**
	 * Initialize this renderer with a generic model
	 */
	UE_API void Initialize(const FViewModelPtr& InViewModel);

	/**
	 * Initialize this renderer with multiple generic models
	 */
	UE_API void Initialize(const TSet<FWeakViewModelPtr>& InViewModels);

	/**
	 * Initialize this renderer with multiple generic models, evaluating partial keys independently per logical group
	 */
	UE_API void InitializeGrouped(const TSet<FWeakViewModelPtr>& InViewModels, TMap<uint32, uint32> InGroupIDsByModelID);

	/**
	 * Update any invalidated data required for drawing keys
	 */
	UE_API void Update(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry) const;

	/**
	 * Draw this batch's curve
	 */
	UE_API int32 DrawCurve(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, const FKeyRendererPaintArgs& Args, int32 LayerId) const;

	/**
	 * Draw this batch
	 */
	UE_API int32 Draw(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, const FKeyRendererPaintArgs& Args, int32 LayerId) const;

	/**
	 * Retrieve keys at the specified position
	 */
	UE_API void HitTestKeys(const FFrameTime& Time, TArray<FKeysForModel>& OutAllKeys, const bool bInLeftAlignCollision = false) const;

	/**
	 * Retrieve keys at the specified position
	 */
	UE_API bool HitTestKeyBar(const FFrameTime& Time, FKeyBar& OutKeyBar) const;

	/**
	 * Returns true if this renderer has a curve, false otherwise
	 */
	UE_API bool HasCurve() const;

private:

	/** Container that caches the key positions for a given key area, along with those that overlap the current visible range */
	struct FCachedKeyDrawInformation
	{
		/** Construction from the key area this represents */
		FCachedKeyDrawInformation(TViewModelPtr<IKeyExtension> KeyExtension, int32 InGroupIndex)
			: NextUnhandledIndex(0)
			, PreserveToIndex(0)
			, GroupIndex(InGroupIndex)
			, WeakKeyExtension(KeyExtension)
		{
			KeyExtension->UpdateCachedKeys(CachedKeys);
		}


		/**
		 * Attempt to update data that is not dependent upon the current view
		 * @return EViewDependentCacheFlags::DataChanged if view-indipendent data was updated, EViewDependentCacheFlags::None otherwise
		 */
		EViewDependentCacheFlags UpdateViewIndependentData();


		/**
		 * Ensure that view-dependent data (such as which keys need drawing and how) is up to date
		 *
		 * @param Params    The batch parameters
		 */
		void CacheViewDependentData(const FKeyBatchParameters& Params);

		/**
		 * Return the size of this key draw information
		 */
		int32 Num() const
		{
			return FramesInRange.Num();
		}

	public:

		/** Index into the array views for the next unhandled key */
		int32 NextUnhandledIndex;

		/** Index into the array views for the first index proceeding a preserved range (TimesInRange.Num() if not) */
		int32 PreserveToIndex;

		/** Logical group used to evaluate partial keys independently when multiple channel groups share one renderer. */
		int32 GroupIndex;

		/** Cached array view retrieved from CachedKeys for the key frames that overlap the current time */
		TArrayView<const FFrameTime> FramesInRange;
		/** Cached array view retrieved from CachedKeys for the key handles that overlap the current time */
		TArrayView<const FKeyHandle> HandlesInRange;
		/** Draw params for each of the keys visible on screen */
		TArray<FKeyDrawParams> DrawParams;

		FKeyDrawParams LeadingKeyParams;
		FKeyDrawParams TrailingKeyParams;

		FCachedKeys::FCachedKey LeadingKey;
		FCachedKeys::FCachedKey TrailingKey;

		/** Construction from the key area this represents */
		TSharedPtr<FCachedKeys> CachedKeys;

		TWeakViewModelPtr<IKeyExtension> WeakKeyExtension;
	};


	/** Cached parameters for drawing a single key */
	struct FKey
	{
		/** Paint parameters for this key */
		FKeyDrawParams Params;

		/** The tick range that this key occupies (significant when this FKey represents multiple overlapping keys) */
		FFrameTime KeyTickStart, KeyTickEnd;

		/** The time that this key should be drawn - represents the average time for overlapping keys */
		FFrameTime FinalKeyPosition;

		/** Flags that specify how to draw this key */
		EKeyRenderingFlags Flags = EKeyRenderingFlags::None;
	};


	/** Cached parameters for drawing a single key */
	struct FCachedKeyBar
	{
		/** The tick range that this key occupies (significant when this FKey represents multiple overlapping keys) */
		FFrameTime StartTime, EndTime;

		/** Flags that specify how to draw this key bar */
		EKeyBarRenderingFlags Flags = EKeyBarRenderingFlags::None;

		/** Key bar connection style */
		EKeyConnectionStyle ConnectionStyle = EKeyConnectionStyle::None;
	};

	/**
	 * Cache all the key extensions from our model
	 */
	UE_API void CacheKeyExtensions(const FKeyBatchParameters& Params) const;

	/**
	 * Attempt to update data that is not dependent upon the current view
	 * @return EViewDependentCacheFlags::DataChanged if view-indipendent data was updated, EViewDependentCacheFlags::None otherwise
	 */
	UE_API EViewDependentCacheFlags UpdateViewIndependentData() const;


	/**
	 * Ensure that view-dependent data (such as which keys need drawing and how) is up to date
	 */
	UE_API void UpdateViewDependentData(const FKeyBatchParameters& Params, const FGeometry& AllottedGeometry) const;

private:

	/** The models that we are drawing keys for */
	TSet<FWeakViewModelPtr> WeakViewModels;

	/** Whether partial-key state should be evaluated independently per logical model group. */
	bool bEvaluatePartialKeysByGroup = false;

	/** Map from model ID to logical partial-key group ID. */
	TMap<uint32, uint32> PartialKeyGroupIDsByModelID;

	/** Array of cached draw info for each of the key areas that comprise this batch */
	mutable TArray<FCachedKeyDrawInformation, TInlineAllocator<1>> KeyDrawInfo;

	/** Computed final draw info for keys */
	mutable TArray<FKey> PrecomputedKeys;

	/** Computed final draw info for key bars */
	mutable TArray<FCachedKeyBar> PrecomputedKeyBars;

	/** Computed final draw curve info */
	mutable TArray<TTuple<double, double>> PrecomputedCurve;

	/** Cached width of a key in frames */
	mutable FFrameTime KeyWidthInFrames;
};

ENUM_CLASS_FLAGS(EViewDependentCacheFlags);
ENUM_CLASS_FLAGS(FKeyRenderer::EKeyRenderingFlags);

} // namespace UE::Sequencer

#undef UE_API
