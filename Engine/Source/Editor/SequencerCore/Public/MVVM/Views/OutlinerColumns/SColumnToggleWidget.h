// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

struct FCreateOutlinerColumnParams;

class IOutlinerColumn;
class FSequencerEditorViewModel;

/**
 * Outliner Column Widget for toggling active state with click and drag functionality.
 * Adjusts image color and transparency based on active state and item hover.
 * Displays an alternate icon when a child is active and the item is not.
 * Column Toggle widgets have 4 States: Active, Inactive, Active Child, and Implicitly Active.
 * Column Toggle widgets will change its brush based on its state.
 */
class SColumnToggleWidget
	: public SImage
{
public:

	SLATE_BEGIN_ARGS(SColumnToggleWidget) {}
	SLATE_END_ARGS()

	UE_API void Construct(
		const FArguments& InArgs,
		const TWeakPtr<IOutlinerColumn> InOutlinerColumn,
		const FCreateOutlinerColumnParams& InParams);

public:

	/** Called when a drag or click ends and can be used to refresh the SequencerTree if necessary. */
	virtual void OnToggleOperationComplete() {}

	/** Get the color and opacity of the column toggle widget. */
	UE_API virtual FSlateColor GetImageColorAndOpacity() const;

protected:

	/** Returns whether the widget is enabled or not. */
	virtual bool IsEnabled() const { return true; }

protected:

	/** Returns whether or not the item is directly active or not. */
	virtual bool IsActive() const = 0;

	/** Sets the active state of this item to the input value. */
	virtual void SetIsActive(const bool bInIsActive) = 0;

	/** Returns true if a child of this item is active. */
	virtual bool IsChildActive() const = 0;

	/** Returns true if this item is implicitly active, but not directly active */
	virtual bool IsImplicitlyActive() const { return false; }

	/** Returns the brush to be used to represent a widget is active. */
	virtual const FSlateBrush* GetActiveBrush() const = 0;

	/** Returns the brush to be used to represent a widget is inactive. If Null, will use the active brush. */
	virtual const FSlateBrush* GetInactiveBrush() const { return nullptr; }

protected:
	

	/** Get the image this widget displays. */
	UE_API virtual const FSlateBrush* GetBrush() const;

protected:

	/** Start a new drag/drop operation for this widget. */
	UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** If a matching column drag drop operation has entered this widget, set its item to the specified active state. */
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** Handles left click-type inputs and potentially begins drag and drop operation. */
	UE_API FReply HandleClick();

	/** Called when the mouse button is double clicked on this widget. */
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	/** Called when the mouse button is pressed down on this widget. */
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	/** Process a mouse up message. */
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Tick this widget to update its cached state */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Called when mouse enters the column widget. */
	UE_API void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Called when mouse leaves the column widget. */
	UE_API void OnMouseLeave(const FPointerEvent& MouseEvent) override;

protected:

	/** Refreshes the Sequencer Tree. Can be used after toggle operation is complete if operation affects other items. */
	UE_API void RefreshSequencerTree();

protected:
	/** Editor view-model for this view. */
	TWeakPtr<FEditorViewModel> WeakEditor;

	/** Outliner item this widget accesses. */
	TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerExtension;

	/** Reference to column this widget is a member of. */
	TWeakPtr<IOutlinerColumn> WeakOutlinerColumn;

private:

	/** Brushes for the various states of active widgets. */
	const FSlateBrush* ActiveBrush = nullptr;
	const FSlateBrush* InactiveBrush = nullptr;
	const FSlateBrush* ChildActiveBrush = nullptr;

	/** Scoped undo transaction. */
	TUniquePtr<FScopedTransaction> UndoTransaction;

protected:

	/** Cached model ID of the outliner widget this widget relates to */
	uint32 ModelID;

private:

	/** Whether or not the mouse is directly over the widget. */
	uint8 bIsMouseOverWidget : 1;

	/** Whether this item is active. */
	uint8 bIsActive : 1;

	/** Whether or not a child of this item is active. */
	uint8 bIsChildActive : 1;

	/** Whether this item is active implicitly (as a result of some item's state, not this item explicitly). */
	uint8 bIsImplicitlyActive : 1;
};

} // namespace UE::Sequencer

#undef UE_API
