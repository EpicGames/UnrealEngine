// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"

#define UE_API SEQUENCERCORE_API

class SMenuAnchor;

namespace UE::Sequencer
{

class IOutlinerExtension;

class SOutlinerColumnButton : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SOutlinerColumnButton) 
		: _Image(nullptr)
		, _UncheckedImage(nullptr)
 		, _IsFocusable(false)
		{}

		SLATE_ATTRIBUTE(const FSlateBrush*, Image)

		SLATE_ATTRIBUTE(const FSlateBrush*, UncheckedImage)

		SLATE_ATTRIBUTE(bool, IsRowHovered)

		SLATE_ATTRIBUTE(bool, IsChecked)

		SLATE_ARGUMENT(bool, IsFocusable)

		SLATE_EVENT(FOnClicked, OnClicked)

		SLATE_EVENT(FOnGetContent, OnGetMenuContent)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:

	UE_API FSlateColor GetColorAndOpacity() const;

	UE_API const FSlateBrush* GetImage() const;

	UE_API FReply ToggleMenu();

private:

	TSharedPtr<SMenuAnchor> MenuAnchor;
	TAttribute<bool> IsRowHovered;
	TAttribute<bool> IsChecked;
	TAttribute<const FSlateBrush*> Image;
	TAttribute<const FSlateBrush*> UncheckedImage;
};

} // namespace UE::Sequencer

#undef UE_API
