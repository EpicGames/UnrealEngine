// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Layout/Children.h"

#define UE_API SEQUENCERCORE_API

struct FGeometry;

class FPaintArgs;
class FSlateWindowElementList;

namespace UE::Sequencer
{

class STrackLane;
class STrackAreaView;
class ITrackLaneWidget;
class ITrackLaneWidgetSpace;

/**
 * 
 */
class SCompoundTrackLaneView
	: public SPanel
{
public:

	SLATE_BEGIN_ARGS(SCompoundTrackLaneView){}
	SLATE_END_ARGS()

	UE_API SCompoundTrackLaneView();
	UE_API ~SCompoundTrackLaneView();

	UE_DEPRECATED(5.6, "Please use the TSharedPtr<ITrackLaneWidgetSpace> overload")
	void Construct(const FArguments& InArgs)
	{
		Construct(InArgs, nullptr);
	}

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<ITrackLaneWidgetSpace> InTrackLaneWidgetSpace);

	UE_API void AddWeakWidget(TSharedPtr<ITrackLaneWidget> InWidget, TWeakPtr<STrackLane> InOwningLane);
	UE_API void AddStrongWidget(TSharedPtr<ITrackLaneWidget> InWidget, TWeakPtr<STrackLane> InOwningLane);

	/*~ SPanel Interface */
	UE_API void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	UE_API FVector2D ComputeDesiredSize(float) const override;
	UE_API FChildren* GetChildren() override;

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:

	struct FSlot : TSlotBase<FSlot>
	{
		FSlot(TSharedPtr<ITrackLaneWidget> InInterface, TWeakPtr<STrackLane> InOwningLane);
		FSlot(TWeakPtr<ITrackLaneWidget> InWeakInterface, TWeakPtr<STrackLane> InOwningLane);

		TSharedPtr<ITrackLaneWidget> GetInterface() const
		{
			return Interface ? Interface : WeakInterface.Pin();
		}

		TSharedPtr<ITrackLaneWidget> Interface;
		TWeakPtr<ITrackLaneWidget> WeakInterface;

		TWeakPtr<STrackLane> WeakOwningLane;
	};

	/** All the widgets in the panel */
	TPanelChildren<FSlot> Children;

	TWeakPtr<ITrackLaneWidgetSpace> WeakTrackLaneWidgetSpace;
};

} // namespace UE::Sequencer

#undef UE_API
