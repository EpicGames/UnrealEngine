// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/FrameTime.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/EnumClassFlags.h"

#define UE_API SEQUENCERCORE_API

struct FFrameNumber;

namespace UE
{
namespace Sequencer
{

class FViewModel;

enum class EKeyNavigationButtons : uint8
{
	PreviousKey = 1 << 0,
	AddKey      = 1 << 1,
	NextKey     = 1 << 2,

	NavOnly = PreviousKey | NextKey,
	All     = NavOnly | AddKey,
};
ENUM_CLASS_FLAGS(EKeyNavigationButtons)

/**
 * A widget for navigating between keys on a sequencer track
 */
class SKeyNavigationButtons
	: public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnSetTime, FFrameTime)
	DECLARE_DELEGATE_OneParam(FOnGetNavigatableTimes, TArray<FFrameNumber>&)
	DECLARE_DELEGATE_TwoParams(FOnAddKey, FFrameTime, TSharedPtr<FViewModel>)

	SLATE_BEGIN_ARGS(SKeyNavigationButtons) : _Buttons(EKeyNavigationButtons::All) {}

		SLATE_ARGUMENT(EKeyNavigationButtons, Buttons)

		SLATE_ARGUMENT(FText, PreviousKeyToolTip)
		SLATE_ARGUMENT(FText, NextKeyToolTip)

		SLATE_ATTRIBUTE(FFrameTime, Time)
		SLATE_EVENT(FOnSetTime, OnSetTime)

		SLATE_EVENT(FOnAddKey, OnAddKey)
		SLATE_ARGUMENT(FText, AddKeyToolTip)

		SLATE_EVENT(FOnGetNavigatableTimes, GetNavigatableTimes)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const TSharedPtr<FViewModel>& InModel);

	UE_API FReply OnPreviousKeyClicked();
	UE_API FReply OnNextKeyClicked();
	UE_API FReply OnAddKeyClicked();

protected:

	TAttribute<FFrameTime> TimeAttribute;
	FOnSetTime SetTimeEvent;
	FOnAddKey AddKeyEvent;
	FOnGetNavigatableTimes GetNavigatableTimesEvent;

	TWeakPtr<FViewModel> WeakModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
