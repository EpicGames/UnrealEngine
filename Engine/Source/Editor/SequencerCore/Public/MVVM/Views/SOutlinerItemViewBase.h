// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "MVVM/ViewModelPtr.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API SEQUENCERCORE_API

class ISequencer;
struct FGeometry;
struct FOptionalSize;
struct FPointerEvent;
struct FSlateBrush;
struct FTableRowStyle;

namespace UE
{
namespace Sequencer
{

class FEditorViewModel;
class IOutlinerExtension;
class IRenameableExtension;
class ISequencerTreeViewRow;

enum class EOutlinerItemViewBaseStyle
{
	Default,
	ContainerHeader,
	InsideContainer,
};

/**
 * A widget for displaying a sequencer tree node in the animation outliner.
 */
class SOutlinerItemViewBase
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SOutlinerItemViewBase) : _ItemStyle(EOutlinerItemViewBaseStyle::Default), _IsReadOnly(false) {}
		SLATE_ARGUMENT(EOutlinerItemViewBaseStyle, ItemStyle)
		SLATE_ATTRIBUTE(bool, IsReadOnly)
		SLATE_NAMED_SLOT(FArguments, CustomContent)
		SLATE_NAMED_SLOT(FArguments, AdditionalLabelContent)
		SLATE_NAMED_SLOT(FArguments, RightGutterContent)
	SLATE_END_ARGS()

	UE_API void Construct( 
			const FArguments& InArgs, 
			TWeakViewModelPtr<IOutlinerExtension> InWeakExtension,
			TWeakPtr<FEditorViewModel> InWeakEditor,
			const TSharedRef<ISequencerTreeViewRow>& InTableRow );

protected:

	/**
	 * @return The color used to draw the display name.
	 */
	UE_API virtual FText GetLabel() const;

	/**
	 * @return The color used to draw the display name.
	 */
	UE_API virtual FSlateColor GetLabelColor() const;

	/**
	*@return The font used to draw the display name.
	*/
	UE_API virtual FSlateFontInfo GetLabelFont() const;

	/**
	 * @return The text displayed for the tool tip for the diplay name label. 
	 */
	UE_API virtual FText GetLabelToolTipText() const;

	/**
	 * Returns whether a requested rename is valid.
	 */
	UE_API bool IsRenameValid(const FText& NewName, FText& OutErrorMessage) const;

	/**
	 * Called when a new name has been committed in the text box.
	 */
	UE_API void OnNodeLabelTextCommitted(const FText& NewLabel, ETextCommit::Type CommitType);

	/**
	 * @return The icon to use on the view
	 */
	UE_API virtual const FSlateBrush* GetIconBrush() const;

	/**
	 * @return A color tint to apply to the icon
	 */
	UE_API virtual FSlateColor GetIconTint() const;

	/**
	 * @return The icon overlay (eg a spawnable badge) to use on the view
	 */
	UE_API virtual const FSlateBrush* GetIconOverlayBrush() const;

	/**
	 * @return Tooltip text to apply to the icon
	 */
	UE_API virtual FText GetIconToolTipText() const;

	/**
	 * Callback for checking whether the node label can be edited.
	 */
	UE_API virtual bool IsNodeLabelReadOnly() const;

	/**
	 * Whether this node should be drawn dimmed or not
	 */
	UE_API virtual bool IsDimmed() const;

	UE_API FOptionalSize GetHeight() const;

private:

	// SWidget interface

	UE_API FSlateColor GetForegroundBasedOnSelection() const;

	/**
	 * @return The border image to show in the tree node.
	 */
	UE_API const FSlateBrush* GetNodeBorderImage() const;

	/**
	* @return The tint to apply to the border image for the inner portion of the node.
	*/
	UE_API FSlateColor GetNodeInnerBackgroundTint() const;

	/**
	 * @return The expander visibility of this node.
	 */
	UE_API EVisibility GetExpanderVisibility() const;

protected:

	/** Weak extension pointer for the outliner (mandatory, non-owning reference) */
	TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerExtension;
	/** Weak extension pointer for a rename extension (optional, non-owning reference) */
	TWeakViewModelPtr<IRenameableExtension> WeakRenameExtension;

	/** Editor view-model for this view. */
	TWeakPtr<FEditorViewModel> WeakEditor;

	TAttribute<bool> IsReadOnlyAttribute;
	TAttribute<bool> IsRowSelectedAttribute;

	/** Default background brush for this node when expanded */
	const FSlateBrush* ExpandedBackgroundBrush;

	/** Default background brush for this node when collapsed */
	const FSlateBrush* CollapsedBackgroundBrush;

	/** The brush to use when drawing the background for the inner portion of the node. */
	const FSlateBrush* InnerBackgroundBrush;

	/** The table row style used for nodes in the tree. This is required as we don't actually use the tree for selection. */
	const FTableRowStyle* TableRowStyle;

	/** Style enumeration */
	EOutlinerItemViewBaseStyle ItemStyle;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
