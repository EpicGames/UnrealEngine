// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Input/Reply.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Views/TreeViewTraits.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API SEQUENCERCORE_API

class FDragDropEvent;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class ITableRow;
class SHeaderRow;
class SScrollBar;
class SWidget;
namespace UE::Sequencer { class FOutlinerViewModel; }
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;
struct FSlateBrush;

namespace UE::Sequencer
{

struct FSelectionEventSuppressor;
struct FOutlinerHeaderRowWidgetMetaData;

class FViewModel;
class FSequencerCoreSelection;
class SOutlinerViewRow;
class STrackAreaView;
class STrackLane;
class IOutlinerColumn;

enum class EOutlinerColumnGroup : uint8;

enum class ETreeRecursion
{
	Recursive, NonRecursive
};


/** The tree view used in the sequencer */
class SOutlinerView
	: public STreeView<TWeakViewModelPtr<IOutlinerExtension>>
{
public:

	SLATE_BEGIN_ARGS(SOutlinerView){}

		SLATE_ARGUMENT( TSharedPtr<FSequencerCoreSelection>, Selection )
		/** Externally supplied scroll bar */
		SLATE_ARGUMENT( TSharedPtr<SScrollBar>, ExternalScrollbar )

	SLATE_END_ARGS()

	static UE_API const FName TrackNameColumn;

	UE_API SOutlinerView();
	UE_API ~SOutlinerView();

	/** Construct this widget */
	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FOutlinerViewModel> InWeakOutliner, const TSharedRef<STrackAreaView>& InTrackArea);
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime );
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	UE_API TSharedPtr<FOutlinerViewModel> GetOutlinerModel() const;

	/** @return the number of root nodes this tree contains */
	int32 GetNumRootNodes() const { return RootNodes.Num(); }

	UE_API float GetVirtualTop() const;

	UE_API void GetVisibleItems(TArray<TViewModelPtr<IOutlinerExtension>>& OutItems) const;

	UE_API void ForceSetSelectedItems(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InItems);

public:

	/** Get the tree item model at the specified physical vertical position */
	UE_API TViewModelPtr<IOutlinerExtension> HitTestNode(float InPhysical) const;

	/** Convert the specified physical vertical position into an absolute virtual position, ignoring expanded states */
	UE_API float PhysicalToVirtual(float InPhysical) const;

	/** Convert the specified absolute virtual position into a physical position in the tree.
	 * @note: Will not work reliably for virtual positions that are outside of the physical space
	 */
	UE_API float VirtualToPhysical(float InVirtual) const;

	UE_API void ReportChildRowGeometry(const TViewModelPtr<IOutlinerExtension>& InNode, const FGeometry& InGeometry);

public:

	/** Refresh this tree as a result of the underlying tree data changing */
	UE_API void Refresh();

	/** Expand or collapse nodes */
	UE_API void ToggleExpandCollapseNodes(ETreeRecursion Recursion = ETreeRecursion::Recursive, bool bExpandAll = false, bool bCollapseAll = false);

	/** Scroll this tree view by the specified number of slate units */
	UE_API void ScrollByDelta(float DeltaInSlateUnits);

	UE_API bool IsColumnVisible(const FName& InName) const;

	/** Set the item's expansion state, including all of its children */
	UE_API void ExpandCollapseNode(TViewModelPtr<IOutlinerExtension> InDataModel, bool bExpansionState, ETreeRecursion Recursion);

protected:

	/** Generate a row for a particular node */
	UE_API virtual TSharedRef<ITableRow> OnGenerateRow(TWeakViewModelPtr<IOutlinerExtension> InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable);

	UE_API void CreateTrackLanesForRow(TSharedRef<SOutlinerViewRow> InRow, TViewModelPtr<IOutlinerExtension> InDataModel);
	UE_API TSharedPtr<STrackLane> FindOrCreateParentLane(TViewModelPtr<IOutlinerExtension> InDataModel);

	/** Gather the children from the specified node */
	UE_API void OnGetChildren(TWeakViewModelPtr<IOutlinerExtension> InParent, TArray<TWeakViewModelPtr<IOutlinerExtension>>& OutChildren) const;

	/** Generate a widget for the specified Node and Column */
	UE_API TSharedRef<SWidget> GenerateWidgetForColumn(TViewModelPtr<IOutlinerExtension> InDataModel, const FName& ColumnId, const TSharedRef<SOutlinerViewRow>& Row) const;

	/** Called when a node has been expanded or collapsed */
	UE_API void OnExpansionChanged(TWeakViewModelPtr<IOutlinerExtension> InItem, bool bIsExpanded);

	// Tree selection methods which must be overriden to maintain selection consistency with the rest of sequencer.
	UE_API virtual void Private_UpdateParentHighlights() override;
	UE_API virtual void Private_SetItemSelection( TWeakViewModelPtr<IOutlinerExtension> TheItem, bool bShouldBeSelected, bool bWasUserDirected = false ) override;
	UE_API virtual void Private_ClearSelection() override;
	UE_API virtual void Private_SelectRangeFromCurrentTo( TWeakViewModelPtr<IOutlinerExtension> InRangeSelectionEnd ) override;
	UE_API virtual void Private_SignalSelectionChanged( ESelectInfo::Type SelectInfo ) override;

	UE_API virtual void OnRightMouseButtonDown(const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnRightMouseButtonUp(const FPointerEvent& MouseEvent) override;

	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

public:

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

protected:

	// Private, unimplemented overloaded name for SetItemSelection to prevent external calls - use ForceSetItemSelection instead
	UE_API void SetItemSelection();
	// Private, unimplemented overloaded name for SetItemSelection to prevent external calls - use ForceSetItemSelection instead
	UE_API void ClearSelection();

	UE_API void UpdateViewSelectionFromModel();
	UE_API void UpdateModelSelectionFromView();

	UE_API void HandleTableViewScrolled(double InScrollOffset);
	UE_API void UpdatePhysicalGeometry(bool bIsRefresh);

	/** Handles the context menu opening when right clicking on the tree view. */
	UE_API TSharedPtr<SWidget> OnContextMenuOpening();

	UE_API void SetItemExpansionRecursive(TWeakViewModelPtr<IOutlinerExtension> InItem, bool bIsExpanded);

public:

	/** Structure used to cache physical geometry for a particular node */
	struct FCachedGeometry
	{
		FCachedGeometry(TWeakViewModelPtr<IOutlinerExtension> InWeakItem,
			float InPhysicalTop, float InPhysicalHeight,
			float InVirtualTop, float InVirtualHeight,
			float InVirtualNestedHeight)
			: WeakItem(MoveTemp(InWeakItem))
			, PhysicalTop(InPhysicalTop), PhysicalHeight(InPhysicalHeight)
			, VirtualTop(InVirtualTop), VirtualHeight(InVirtualHeight), VirtualNestedHeight(InVirtualNestedHeight)
		{}

		TWeakViewModelPtr<IOutlinerExtension> WeakItem;
		float PhysicalTop, PhysicalHeight;
		float VirtualTop, VirtualHeight, VirtualNestedHeight;
	};

	/** Access all the physical nodes currently visible on the sequencer */
	const TArray<FCachedGeometry>& GetAllVisibleNodes() const { return PhysicalNodes; }

	/** Add a SOutlinerView object that should be modified or updated when this Treeview is updated */
	UE_API void AddPinnedTreeView(TSharedPtr<SOutlinerView> PinnedTreeView);

	/** Set a SOutlinerView object this Treeview is pinned to, for operations that should happen on the primary */
	void SetPrimaryTreeView(TSharedPtr<SOutlinerView> InPrimaryTreeView) { PrimaryTreeView = InPrimaryTreeView; }

	/** Set whether this TreeView should show only pinned nodes or only non-pinned nodes  */
	void SetShowPinned(bool bShowPinned) { bShowPinnedNodes = bShowPinned; }

	/** Updates the list of visible outliner columns and regenerates columns in the outliner view */
	UE_API void SetOutlinerColumns(const TArray<TSharedPtr<IOutlinerColumn>>& InOutlinerColumns);

protected:

	/** Linear, sorted array of nodes that we currently have generated widgets for */
	TArray<FCachedGeometry> PhysicalNodes;

	UE_API int32 CreateOutlinerColumnsForGroup(int32 ColumnIndex, EOutlinerColumnGroup Group);

	/** Populate the map of column definitions, and add relevant columns to the header row. Must be called when outliner columns change */
	UE_API void UpdateOutlinerColumns();

	/** Insert a separator column at the specified column index, with a unique identifier */
	UE_API void InsertSeparatorColumn(int32 InsertIndex, int32 SeparatorID);

	UE_API FReply OnDragRow(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, TSharedRef<SOutlinerViewRow> InRow);

	UE_API FString OnItemToString_Debug(TWeakViewModelPtr<IOutlinerExtension> InWeakModel);

protected:

	using FColumnGenerator = TFunction<TSharedPtr<SWidget>(const FCreateOutlinerColumnParams& Params, const TSharedRef<SOutlinerViewRow>&)>;

	/** The tree view's header row (hidden) */
	TSharedPtr<SHeaderRow> HeaderRow;

	/** MetaData pertaining to each column within HeaderRow */
	TSharedPtr<FOutlinerHeaderRowWidgetMetaData> ColumnMetaData;

	/** The outliner view model */
	TWeakPtr<FOutlinerViewModel> WeakOutliner;

	/** Cached copy of the root nodes from the tree data */
	TArray<TWeakViewModelPtr<IOutlinerExtension>> RootNodes;

	/** Column definitions for each of the columns in the tree view */
	TMap<FName, FColumnGenerator> ColumnGenerators;

	TSharedPtr<FSequencerCoreSelection> Selection;
	TUniquePtr<FSelectionEventSuppressor> DelayedEventSuppressor;

	/** Strong pointer to the track area so we can generate track lanes as we need them */
	TSharedPtr<STrackAreaView> TrackArea;

	/** SOutlinerView objects that should be modified or updated when this Treeview is updated */
	TArray<TSharedPtr<SOutlinerView>> PinnedTreeViews;

	/** The SOutlinerView object this SOutlinerView is pinned to, or nullptr if not pinned */
	TWeakPtr<SOutlinerView> PrimaryTreeView;

	/** Visible Outliner columns to display in the outliner view */
	TArray<TSharedPtr<IOutlinerColumn>> OutlinerColumns;

	float VirtualTop;

	/** When true, the tree selection is being updated from a change in the sequencer selection. */
	bool bUpdatingTreeSelection;

	/** Right mouse button is down, don't update sequencer selection. */
	bool bRightMouseButtonDown;

	/** Whether this tree is for pinned nodes or non-pinned nodes */
	bool bShowPinnedNodes;

	/** Whether we have pending selection changes to broadcast */
	bool bSelectionChangesPending;

	/** Whether physical geometry information should be recomputed */
	bool bRefreshPhysicalGeometry;
};

} // namespace UE::Sequencer

#undef UE_API
