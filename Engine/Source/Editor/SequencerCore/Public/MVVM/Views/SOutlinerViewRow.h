// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "SequencerCoreFwd.h"
#include "MVVM/Extensions/IOutlinerExtension.h"

#define UE_API SEQUENCERCORE_API

namespace UE::Sequencer
{

class STrackLane;

/**
 * Widget that represents a row in the sequencer's tree control.
 */
class SOutlinerViewRow
	: public ISequencerTreeViewRow
{
public:
	DECLARE_DELEGATE_RetVal_ThreeParams(TSharedRef<SWidget>, FOnGenerateWidgetForColumn, TViewModelPtr<IOutlinerExtension>, const FName&, const TSharedRef<SOutlinerViewRow>&);
	DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FDetectDrag, const FGeometry&, const FPointerEvent&, TSharedRef<SOutlinerViewRow>);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsColumnVisible, const FName&);

	SLATE_BEGIN_ARGS(SOutlinerViewRow){}

		/** Delegate to invoke to retrieve a specific column's visibility */
		SLATE_EVENT(FIsColumnVisible, OnGetColumnVisibility)

		/** Delegate to invoke to create a new column for this row */
		SLATE_EVENT(FOnGenerateWidgetForColumn, OnGenerateWidgetForColumn)

		/** Detect a drag on this tree row */
		SLATE_EVENT(FDetectDrag, OnDetectDrag)

	SLATE_END_ARGS()

	/** Construct function for this widget */
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakViewModelPtr<IOutlinerExtension> InDataModel);

	/** Destroy this widget */
	UE_API ~SOutlinerViewRow();

	/** Get the model to which this row relates */
	UE_API TViewModelPtr<IOutlinerExtension> GetDataModel() const;

	/**
	 * Gets the track lane we relate to
	 * @param bOnlyOwnTrackLane  Whether to return nullptr if our referenced track lane wasn't created
	 *							 by our own outliner item view-model 
	 * @return The track lane for this row
	 */
	UE_API TSharedPtr<STrackLane> GetTrackLane(bool bOnlyOwnTrackLane = false) const;

	/**
	 * Adds a reference to track lane, either because:
	 *  - It is a parent row's track lane that we want to be kept alive when the parent row disappears out of view, or
	 *  - It is a track lane that was created by our outliner view-model
	 */
	UE_API void SetTrackLane(const TSharedPtr<STrackLane>& InTrackLane);

	/** Whether the underlying data model is selectable */
	UE_API bool IsSelectable() const;

	UE_API virtual const FSlateBrush* GetBorder() const override;

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	UE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	UE_API virtual int32 OnPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	UE_API virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override;

	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	UE_API virtual bool IsColumnVisible(const FName& InColumnName) const override;

	/** Called whenever a drag is detected by the tree view. */
	UE_API FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	/** Called to determine whether a current drag operation is valid for this row. */
	UE_API TOptional<EItemDropZone> OnCanAcceptDrop( const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TWeakViewModelPtr<IOutlinerExtension> InDataModel);

	/** Called to complete a drag and drop onto this drop. */
	UE_API FReply OnAcceptDrop( const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TWeakViewModelPtr<IOutlinerExtension> InDataModel);

protected:

	UE_API TSharedPtr<SWidget> GenerateWidgetForColumn(const FName& ColumnId);

	UE_API FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

protected:

	/**
	 * Cached reference to a track lane that we relate to.
	 * Depending on the situation, this is either the actual track lane originally created
	 * by the our outliner model, or it's just a reference we use to keep the track lane alive
	 * (it's a weak widget) as long as we are in view.
	 */
	TSharedPtr<STrackLane> TrackLane;

	/** The item associated with this row of data */
	TWeakViewModelPtr<IOutlinerExtension> WeakModel;

	/** Delegate to call to create a new widget for a particular column. */
	FOnGenerateWidgetForColumn OnGenerateWidgetForColumn;

	/** Delegate to call when dragging a tree item is detected */
	FDetectDrag OnDetectDrag;

	/** Delegate to call to retrieve column visibility. */
	FIsColumnVisible OnGetColumnVisibility;

	float SeparatorHeight;
};

} // namespace UE::Sequencer

#undef UE_API
