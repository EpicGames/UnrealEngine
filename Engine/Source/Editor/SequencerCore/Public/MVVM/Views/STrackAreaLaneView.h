// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"

#define UE_API SEQUENCERCORE_API


namespace UE::Sequencer
{

class STrackAreaView;

class STrackAreaLaneView
	: public SCompoundWidget
	, public ITrackLaneWidget
{
public:
	SLATE_BEGIN_ARGS(STrackAreaLaneView){}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FViewModelPtr& InViewModel, TSharedPtr<STrackAreaView> InTrackAreaView);

	/*~ ITrackLaneWidget */
	UE_API virtual TSharedRef<const SWidget> AsWidget() const;
	UE_API virtual FTrackLaneScreenAlignment GetAlignment(const ITrackLaneWidgetSpace& InScreenSpace, const FGeometry& InParentGeometry) const;

	UE_API FTimeToPixel GetRelativeTimeToPixel() const;

protected:

	FWeakViewModelPtr WeakModel;
	TWeakPtr<STrackAreaView> WeakTrackAreaView;
	TSharedPtr<FTimeToPixel> TrackAreaTimeToPixel;
};

} // namespace UE::Sequencer

#undef UE_API
