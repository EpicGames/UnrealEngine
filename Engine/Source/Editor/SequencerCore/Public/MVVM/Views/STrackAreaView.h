// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Geometry.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Views/SequencerInputHandlerStack.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"

#define UE_API SEQUENCERCORE_API

class FArrangedChildren;
class FChildren;
class FDragDropEvent;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class ISequencerEditTool;
struct FCaptureLostEvent;
struct FFrameNumber;
struct FGeometry;
struct FPointerEvent;
template <typename ElementType> class TRange;
struct FTimeToPixel;

namespace UE
{
namespace Sequencer
{

class FTrackAreaViewModel;
class FTrackAreaViewSpace;
class FViewModel;
class IOutlinerExtension;
class SOutlinerView;
class STrackLane;

/**
 * Structure representing a slot in the track area.
 */
class FTrackAreaSlot
	: public TSlotBase<FTrackAreaSlot>, public TAlignmentWidgetSlotMixin<FTrackAreaSlot>
{
public:

	/** Construction from a track lane */
	FTrackAreaSlot(const TSharedPtr<STrackLane>& InSlotContent);

	/** The track lane that we represent. */
	TWeakPtr<STrackLane> TrackLane;
};


struct FTrackAreaViewLayers
{
	int32 LaneBackgrounds = 0;
};


/**
 * The area where tracks( rows of sections ) are displayed.
 */
class STrackAreaView
	: public SPanel
	, public ITrackLaneWidgetSpace
{
public:

	SLATE_BEGIN_ARGS( STrackAreaView )
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
	SLATE_END_ARGS()

	UE_API STrackAreaView();
	UE_API ~STrackAreaView();

	/** Construct this widget */
	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FTrackAreaViewModel> InWeakViewModel);

	UE_API TSharedPtr<FTrackAreaViewModel> GetViewModel() const;
	
	UE_API void SetVirtualPosition(float InVirtualTop);

	UE_API TSharedPtr<FTimeToPixel> GetFallbackTimeToPixel() const;
	UE_API TSharedPtr<FTimeToPixel> GetTimeToPixel(const FGuid& InViewSpaceID = FGuid()) const;

public:

	/** Empty the track area */
	UE_API void Empty();

	/** Drop slots whose track lane has been destroyed because no row holds it any more. */
	UE_API void PurgeDeadTrackSlots();

	/** Add a new track slot to this area for the given node. The slot will be automatically cleaned up when all external references to the supplied slot are removed. */
	UE_API void AddTrackSlot(const TViewModelPtr<IOutlinerExtension>& InDataModel, const TSharedPtr<STrackLane>& InSlot);

	/** Attempt to find an existing slot relating to the given node */
	UE_API TSharedPtr<STrackLane> FindTrackSlot(const TViewModelPtr<IOutlinerExtension>& InDataModel);

	/** Assign a tree view to this track area. */
	UE_API void SetOutliner(const TSharedPtr<SOutlinerView>& InOutliner);
	TWeakPtr<SOutlinerView> GetOutliner() const { return WeakOutliner.Pin(); }

	/** Set whether this TrackArea should show only pinned nodes or only non-pinned nodes  */
	void SetShowPinned(bool bShowPinned) { bShowPinnedNodes = bShowPinned; }
	bool ShowPinned() const { return bShowPinnedNodes; }

	/** Set whether this TrackArea is pinned to another TrackArea and should skip updating external controls */
	void SetIsPinned(bool bInIsPinned) { bIsPinned = bInIsPinned; }
	bool IsPinned() const { return bIsPinned; }

	UE_API const FTrackAreaViewLayers& GetPaintLayers() const;
	static UE_API FLinearColor BlendDefaultTrackColor(FLinearColor InColor);

public:

	/*~ SWidget interface */
	UE_API FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API FReply OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	UE_API int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	UE_API FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	UE_API void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UE_API void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	UE_API FVector2D ComputeDesiredSize(float) const override;
	UE_API FChildren* GetChildren() override;
	UE_API void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	UE_API FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/*~ Begin ITrackLaneWidgetSpace Interface */
	UE_API virtual FTimeToPixel GetScreenSpace(const FGuid& InViewSpaceID = FGuid()) const override;
	/*~ End ITrackLaneWidgetSpace Interface */

protected:

	virtual void OnResized(const FVector2D& OldSize, const FVector2D& NewSize){}

	/** Update any hover state required for the track area */
	UE_API virtual void UpdateHoverStates( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );

private:

	UE_API void HandleViewSpaceAdded(const FGuid& InID, TSharedPtr<FTrackAreaViewSpace> ViewSpace);
	UE_API void HandleViewSpaceRemoved(const FGuid& InID, TSharedPtr<FTrackAreaViewSpace> ViewSpace);

private:

	/** The track area's children. */
	TPanelChildren<FTrackAreaSlot> Children;

protected:

	/** Input handler stack responsible for routing input to the different handlers */
	FInputHandlerStack InputStack;

	/** A map of child slot content that exist in our view. */
	TMap<TWeakViewModelPtr<IOutlinerExtension>, TWeakPtr<STrackLane>> TrackSlots;

	/** Weak pointer to the track area view model. */
	TWeakPtr<FTrackAreaViewModel> WeakViewModel;

	/** Weak pointer to the outliner view (used for scrolling interactions). */
	TWeakPtr<SOutlinerView> WeakOutliner;

	/** Keep a hold of the size of the area so we can maintain zoom levels. */
	TOptional<FVector2D> SizeLastFrame;

	/** Weak pointer to the dropped node */
	TWeakViewModelPtr<IOutlinerExtension> WeakDroppedItem;

	mutable TSharedPtr<FTimeToPixel> TimeToPixel;
	mutable TMap<FGuid, TSharedPtr<FTimeToPixel>> ViewSpaceTimeToPixel;

	float VirtualTop;

	mutable FTrackAreaViewLayers LayerIds;

	/** The frame range of the section about to be dropped */
	TOptional<TRange<FFrameNumber>> DropFrameRange;

	/** Whether the dropped node is allowed to be dropped onto */
	bool bAllowDrop;

	/** Whether this TrackArea is for pinned nodes or non-pinned nodes */
	bool bShowPinnedNodes;

	/** Whether this TrackArea is pinned to another TrackArea and should skip updating external controls */
	bool bIsPinned;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
