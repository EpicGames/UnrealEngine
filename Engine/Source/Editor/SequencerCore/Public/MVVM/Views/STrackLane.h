// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Layout/Children.h"

#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"

#define UE_API SEQUENCERCORE_API

class FPaintArgs;
class FSlateWindowElementList;

namespace UE
{
namespace Sequencer
{

class SOutlinerView;
class STrackAreaView;

/**
 * A wrapper widget responsible for positioning a track lane within the section area
 */
class STrackLane
	: public SPanel
{
public:
	SLATE_BEGIN_ARGS(STrackLane){}
	SLATE_END_ARGS()

	UE_API STrackLane();
	UE_API ~STrackLane();

	/** Construct this widget */
	UE_API void Construct(const FArguments& InArgs, TWeakPtr<STrackAreaView> InTrackAreaView, TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerItem, TSharedPtr<STrackLane> InParentLane, const FTrackAreaParameters& InTrackParams, const TSharedRef<SOutlinerView>& InTreeView);

	/*~ SWidget Interface */
	UE_API int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	/*~ SPanel Interface */
	UE_API void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	UE_API FVector2D ComputeDesiredSize(float) const override;
	FChildren* GetChildren() override { return &Children; }

	/** Gets the outliner view-model that created this track lane */
	UE_API TViewModelPtr<IOutlinerExtension> GetOutlinerItem() const;

	UE_API TSharedPtr<STrackAreaView> GetTrackAreaView() const;
	UE_API TSharedPtr<STrackLane> GetParentLane() const;
	UE_API TSharedPtr<FTimeToPixel> GetTimeToPixel() const;

	UE_API TArrayView<const TWeakPtr<STrackLane>> GetChildLanes() const;

	const TMap<FWeakViewModelPtr, TSharedPtr<ITrackLaneWidget>>& GetAllWidgets() const { return WidgetsByModel; }

	/** Gets the vertical offset of this track lane */
	UE_API float GetVerticalPosition() const;
	/** Sets the vertical offset of this track lane */
	UE_API void SetVerticalPosition(float InPosition);

	/**
	 * Sets the vertical offset of this track lane and all parent track lanes given the vertical offset of
	 * a particular outliner item inside the current track lane (i.e. an outliner item that's a child of the 
	 * item that created this track lane)
	 * Note that the given item *could* actually be the one that created this track lane.
	 */
	UE_API void PositionParentTrackLanes(TViewModelPtr<IOutlinerExtension> InItem, float InItemTop);

	/** Returns whether the outliner item that created this track lane is pinned or not */
	UE_API bool IsPinned() const;

	UE_API TSharedPtr<ITrackLaneWidget> FindWidgetForModel(const FWeakViewModelPtr& InModel) const;

protected:

	UE_API int32 PaintLaneForeground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) const;

	UE_API void OnHierarchyUpdated();
	UE_API void RecreateWidgets();

private:
	TWeakPtr<STrackAreaView> WeakTrackAreaView;

	/** The outliner item that created this lane */
	TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerItem;

	/** Pointer back to the tree view for virtual <-> physical space conversions. Important: weak ptr to avoid circular references */
	TWeakPtr<SOutlinerView> TreeView;

	TMap<FWeakViewModelPtr, TSharedPtr<ITrackLaneWidget>> WidgetsByModel;

	TArray<TWeakPtr<STrackLane>> WeakChildLanes;

	/** Parent track lane - intentionally a _strong_ ptr to the parent in order to keep it alive even if it is scrolled out of view */
	TSharedPtr<STrackLane> ParentLane;

	TSharedPtr<FTimeToPixel> TimeToPixel;

	struct FSlot : TSlotBase<FSlot>
	{
		FSlot(TSharedPtr<ITrackLaneWidget> InInterface);
		~FSlot();

		TSharedRef<ITrackLaneWidget> Interface;
	};

	/** All the widgets in the panel */
	TPanelChildren<FSlot> Children;

	/** Our desired size last frame */
	TOptional<FVector2D> LastDesiredSize;

	struct FDragParameters
	{
		FDragParameters(float InOriginalHeight, float InDragStartY) : OriginalHeight(InOriginalHeight), DragStartY(InDragStartY) {}

		float OriginalHeight;
		float DragStartY;
	};
	TOptional<FDragParameters> DragParameters;

	FTrackAreaParameters TrackParams;

	float Position;

	bool bWidgetsDirty;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
