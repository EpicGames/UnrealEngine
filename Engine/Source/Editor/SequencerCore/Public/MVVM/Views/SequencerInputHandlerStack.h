// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Input/Reply.h"
#include "Delegates/Delegate.h"
#include "ISequencerInputHandler.h"

class SWidget;

namespace UE
{
namespace Sequencer
{

/**
 * Class responsible for handling input to multiple objects
 * that reside at the same level in the widget hierarchy.
 *
 * The sequencer track area is one such example of a single widget
 * that delegates its input handling to multiple sources (edit tool, or time slider controller).
 * To alleviate the complexity of handling input for such sources,
 * where each may fight for mouse capture, this class keeps track
 * of which handler captured the mouse and routs input accordingly.
 *
 * When no mouse capture is active, handlers are called sequentially
 * in the order they were added, until the event is handled.
 */
class FInputHandlerStack
{
public:

	FSimpleMulticastDelegate OnBeginCapture;
	FSimpleMulticastDelegate OnEndCapture;

	FInputHandlerStack() : CapturedIndex(INDEX_NONE) {}

	/** Add a handler to the stack */
	void AddHandler(ISequencerInputHandler* Handler) { Handlers.Add(Handler); }

	/** Reset an existing entry in the stack to a new handler */
	void SetHandlerAt(int32 Index, ISequencerInputHandler* Handler)
	{
		if (Handlers.IsValidIndex(Index))
		{
			if (Handlers[Index] != Handler)
			{
				if (CapturedIndex != INDEX_NONE)
				{
					CapturedIndex = INDEX_NONE;
					OnEndCapture.Broadcast();
				}
			}
			
			Handlers[Index] = Handler;
		}
	}

	/** Get the index of the currently captured handler, or INDEX_NONE */
	int32 GetCapturedIndex() const { return CapturedIndex; }

	/** Handle a mouse down */
	FReply HandleMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return ProcessEvent(&ISequencerInputHandler::OnMouseButtonDown, OwnerWidget, MyGeometry, MouseEvent);
	}

	/** Handle a mouse up */
	FReply HandleMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return ProcessEvent(&ISequencerInputHandler::OnMouseButtonUp, OwnerWidget, MyGeometry, MouseEvent);
	}

	/** Handle a mouse double click */
	FReply HandleMouseDoubleClick(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return ProcessEvent(&ISequencerInputHandler::OnMouseButtonDoubleClick, OwnerWidget, MyGeometry, MouseEvent);
	}

	/** Handle a mouse move */
	FReply HandleMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return ProcessEvent(&ISequencerInputHandler::OnMouseMove, OwnerWidget, MyGeometry, MouseEvent);
	}

	/** Handle a mouse wheel */
	FReply HandleMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return ProcessEvent(&ISequencerInputHandler::OnMouseWheel, OwnerWidget, MyGeometry, MouseEvent);
	}

	FReply HandleKeyDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		return ProcessKeyEvent(&ISequencerInputHandler::OnKeyDown, OwnerWidget, MyGeometry, InKeyEvent);
	}
	
	FReply HandleKeyUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		return ProcessKeyEvent(&ISequencerInputHandler::OnKeyUp, OwnerWidget, MyGeometry, InKeyEvent);
	}
private:

	typedef FReply(ISequencerInputHandler::*InputHandlerFunction)(SWidget&, const FGeometry&, const FPointerEvent&);
	typedef FReply(ISequencerInputHandler::* KeyInputHandlerFunction)(SWidget&, const FGeometry&, const FKeyEvent&);

	FReply ProcessEvent(InputHandlerFunction Function, SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		FReply Reply = FReply::Unhandled();

		// Give the captured index priority over everything
		if (CapturedIndex != INDEX_NONE && Handlers[CapturedIndex])
		{
			Reply = (Handlers[CapturedIndex]->*Function)(OwnerWidget, MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return ProcessReply(Reply, CapturedIndex);
			}
		}

		for (int32 Index = 0; Index < Handlers.Num(); ++Index)
		{
			if (!Handlers[Index] || CapturedIndex == Index)
			{
				continue;
			}

			Reply = (Handlers[Index]->*Function)(OwnerWidget, MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return ProcessReply(Reply, Index);
			}
		}

		return Reply;
	}

	const FReply& ProcessReply(const FReply& Reply, int32 ThisIndex)
	{
		if (Reply.GetMouseCaptor().IsValid())
		{
			CapturedIndex = ThisIndex;
			OnBeginCapture.Broadcast();
		}
		else if (Reply.ShouldReleaseMouse())
		{
			CapturedIndex = INDEX_NONE;
			OnEndCapture.Broadcast();
		}
		return Reply;
	}

	//use mouse capture for the key event
	FReply ProcessKeyEvent(KeyInputHandlerFunction Function, SWidget& OwnerWidget, const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
	{
		FReply Reply = FReply::Unhandled();

		// Give the captured index priority over everything
		if (CapturedIndex != INDEX_NONE && Handlers[CapturedIndex])
		{
			Reply = (Handlers[CapturedIndex]->*Function)(OwnerWidget, MyGeometry, KeyEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}

		for (int32 Index = 0; Index < Handlers.Num(); ++Index)
		{
			if (!Handlers[Index] || CapturedIndex == Index)
			{
				continue;
			}

			Reply = (Handlers[Index]->*Function)(OwnerWidget, MyGeometry, KeyEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}
		}

		return Reply;
	}

	/** Index of the handler that currently has the mouse captured */
	int32 CapturedIndex;

	/** Array o9f input handlers */
	TArray<ISequencerInputHandler*> Handlers;
};

} // namespace Sequencer
} // namespace UE

