// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "MVVM/ViewModels/ViewModel.h"

template<typename ExtensionType>
struct TListTypeTraits<UE::Sequencer::TWeakViewModelPtr<ExtensionType>>
{
public:
	using PtrType = UE::Sequencer::TWeakViewModelPtr<ExtensionType>;
	using NullableType = PtrType;

	using MapKeyFuncs       = TDefaultMapHashableKeyFuncs<PtrType, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<PtrType, FSparseItemInfo, false>;
	using SetKeyFuncs       = DefaultKeyFuncs<PtrType>;

	template<typename U>
	static void AddReferencedObjects( FReferenceCollector&, 
		TArray< PtrType >&, 
		TSet< PtrType >&, 
		TMap< const U*, PtrType >& )
	{}

	static bool IsPtrValid( const PtrType& InPtr )
	{
		return (bool)InPtr.Pin();
	}

	static void ResetPtr( PtrType& InPtr )
	{
		InPtr = nullptr;
	}

	static PtrType MakeNullPtr()
	{
		return PtrType();
	}

	static PtrType NullableItemTypeConvertToItemType( const PtrType& InPtr )
	{
		return InPtr;
	}

	static FString DebugDump( PtrType InPtr )
	{
		UE::Sequencer::TViewModelPtr<ExtensionType> Pinned = InPtr.Pin();
		return Pinned ? FString::Printf(TEXT("0x%08" UPTRINT_x_FMT), (UPTRINT)Pinned.AsModel().Get()) : FString(TEXT("nullptr"));
	}

	class SerializerType{};
};

template <typename ExtensionType>
struct TIsValidListItem<UE::Sequencer::TWeakViewModelPtr<ExtensionType>>
{
	enum
	{
		Value = true
	};
};
