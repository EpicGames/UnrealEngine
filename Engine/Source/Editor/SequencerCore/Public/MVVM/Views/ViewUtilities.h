// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "SequencerCoreFwd.h"
#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointer.h"


struct FSlateBrush;
class FText;
class SWidget;
template< typename ObjectType > class TAttribute;

namespace UE::Sequencer
{

SEQUENCERCORE_API TSharedRef<SWidget> MakeButton(FText HoverText, const FSlateBrush* Image, const FOnClicked& HandleClicked, const FViewModelPtr& ViewModel);

SEQUENCERCORE_API TSharedRef<SWidget> MakeButton(FText HoverText, const FSlateBrush* Image, const FOnGetContent& HandleGetMenuContent, const FViewModelPtr& ViewModel);

SEQUENCERCORE_API TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnClicked& HandleClicked, const FViewModelPtr& ViewModel);

SEQUENCERCORE_API TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnGetContent& HandleGetMenuContent, const FViewModelPtr& ViewModel);

SEQUENCERCORE_API TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnClicked& OnClicked, const TAttribute<bool>& HoverState, const TAttribute<bool>& IsEnabled);

SEQUENCERCORE_API TSharedRef<SWidget> MakeAddButton(FText HoverText, const FOnGetContent& HandleGetMenuContent, const TAttribute<bool>& HoverState, const TAttribute<bool>& IsEnabled);

} // namespace UE::Sequencer