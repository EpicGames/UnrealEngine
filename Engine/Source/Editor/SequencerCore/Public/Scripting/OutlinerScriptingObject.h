// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "MVVM/ViewModelPtr.h"

#include "OutlinerScriptingObject.generated.h"

namespace UE::Sequencer
{
	class FOutlinerViewModel;
}

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FSequencerOutlinerSelectionChanged);

UCLASS(MinimalAPI)
class USequencerOutlinerScriptingObject : public UObject
{
public:

	GENERATED_BODY()

	UPROPERTY(BlueprintAssignable, Category="Sequencer Editor")
	FSequencerOutlinerSelectionChanged OnSelectionChanged;

	SEQUENCERCORE_API void Initialize(UE::Sequencer::TViewModelPtr<UE::Sequencer::FOutlinerViewModel> InOutliner);

	UFUNCTION(BlueprintCallable, Category="Sequencer Editor")
	FSequencerViewModelScriptingStruct GetRootNode() const;

	UFUNCTION(BlueprintCallable, Category="Sequencer Editor")
	TArray<FSequencerViewModelScriptingStruct> GetChildren(FSequencerViewModelScriptingStruct Node, FName TypeName = NAME_None) const;

	UFUNCTION(BlueprintCallable, Category="Sequencer Editor")
	TArray<FSequencerViewModelScriptingStruct> GetSelection() const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void SetSelection(const TArray<FSequencerViewModelScriptingStruct>& InSelection);

	// Mute
	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	TArray<FSequencerViewModelScriptingStruct> GetMuteNodes();

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void SetMuteNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInMuted);

	// Solo
	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	TArray<FSequencerViewModelScriptingStruct> GetSoloNodes();

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void SetSoloNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInSoloed);

	// Deactivated
	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	TArray<FSequencerViewModelScriptingStruct> GetDeactivatedNodes();

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void SetDeactivatedNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInDeactivated);

	// Locked
	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	TArray<FSequencerViewModelScriptingStruct> GetLockedNodes();

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void SetLockedNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInLocked);

	// Pinned
	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	TArray<FSequencerViewModelScriptingStruct> GetPinnedNodes();

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void SetPinnedNodes(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bInPinned);

	// Expansion
	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void SetExpansion(const TArray<FSequencerViewModelScriptingStruct>& InNodes, bool bExpanded);

	UFUNCTION(BlueprintPure, Category = "Sequencer Editor")
	bool IsExpanded(FSequencerViewModelScriptingStruct InNode) const;

protected:

	UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FOutlinerViewModel> WeakOutliner;

private:

	void BroadcastSelectionChanged();
};