// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"

#include "SequencerScriptingLayer.generated.h"

class USequencerOutlinerScriptingObject;

namespace UE::Sequencer
{
	class FEditorViewModel;
}

UCLASS(MinimalAPI)
class USequencerScriptingLayer : public UObject
{
public:

	GENERATED_BODY()

	SEQUENCERCORE_API virtual void Initialize(TSharedPtr<UE::Sequencer::FEditorViewModel> InViewModel);

	UPROPERTY(BlueprintReadOnly, Category="Sequencer Editor")
	TObjectPtr<USequencerOutlinerScriptingObject> Outliner;
};