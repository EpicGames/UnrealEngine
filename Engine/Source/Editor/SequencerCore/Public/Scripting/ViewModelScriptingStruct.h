// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModelPtr.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "ViewModelScriptingStruct.generated.h"


USTRUCT(BlueprintType)
struct FSequencerViewModelScriptingStruct
{
	GENERATED_BODY()

	SEQUENCERCORE_API FSequencerViewModelScriptingStruct();

	SEQUENCERCORE_API FSequencerViewModelScriptingStruct(UE::Sequencer::FViewModelPtr InViewModel);

	friend bool operator==(const FSequencerViewModelScriptingStruct& A, const FSequencerViewModelScriptingStruct& B)
	{
		return A.WeakViewModel == B.WeakViewModel;
	}
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Sequencer Editor")
	FName Type;

	UE::Sequencer::FWeakViewModelPtr WeakViewModel;
};

template<>
struct TStructOpsTypeTraits<FSequencerViewModelScriptingStruct> : public TStructOpsTypeTraitsBase2<FSequencerViewModelScriptingStruct>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithCopy = true,
	};
};


/**
 * Function library containing methods that should be hoisted onto FSequencerScriptingRanges
 */
UCLASS()
class USequencerViewModelStructExtensions : public UBlueprintFunctionLibrary
{
public:

	GENERATED_BODY()

	UFUNCTION(BlueprintPure, Category = "Sequencer Editor", meta=(ScriptMethod))
	static FString GetLabel(const FSequencerViewModelScriptingStruct& ViewModel);

	UFUNCTION(BlueprintPure, Category = "Sequencer Editor", meta = (ScriptMethod))
	static FString GetPath(const FSequencerViewModelScriptingStruct& ViewModel);
};