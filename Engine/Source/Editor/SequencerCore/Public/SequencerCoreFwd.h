// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE
{
namespace Sequencer
{

/**
 * View model type forward declarations
 */
class ICastable;
class FViewModel;
class FSharedViewModelData;
class FViewModelHierarchyOperation;
class FScopedViewModelHierarchyOperation;

struct FViewModelListHead;
struct FViewModelChildren;

/**
 * Iterator forward declarations - include MVVM/ViewModels/ViewModelIterators.h for complete declarations
 */
struct FParentModelIterator;
struct FParentFirstChildIterator;
struct FViewModelListIterator;
struct FViewModelSubListIterator;
struct FViewModelIterationState;

template<typename, typename> struct TTypedIterator;

template<typename T>
using TParentModelIterator = TTypedIterator<T, FParentModelIterator>;
template<typename T>
using TParentFirstChildIterator = TTypedIterator<T, FParentFirstChildIterator>;
template<typename T>
using TViewModelListIterator = TTypedIterator<T, FViewModelListIterator>;
template<typename T>
using TViewModelSubListIterator = TTypedIterator<T, FViewModelSubListIterator>;


/**
 * Shared/weak extension pointer forward declarations - include MVVM/ViewModelPtr.h for complete declarations
 */
template<typename, bool>
struct TImplicitViewModelCastImpl;
template<typename>
struct TImplicitWeakViewModelPin;

template<typename ViewModelType>
using TImplicitViewModelCast = TImplicitViewModelCastImpl<ViewModelType, false>;

template<typename ViewModelType>
using TImplicitViewModelCastChecked = TImplicitViewModelCastImpl<ViewModelType, true>;

template<typename>
struct TViewModelPtr;

template<typename>
struct TWeakViewModelPtr;

using FViewModelPtr = TViewModelPtr<FViewModel>;
using FConstViewModelPtr = TViewModelPtr<const FViewModel>;
using FWeakViewModelPtr = TWeakViewModelPtr<FViewModel>;

} // namespace Sequencer
} // namespace UE

