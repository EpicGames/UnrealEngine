// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Layout/Geometry.h"
#include "Misc/FrameRate.h"

namespace UE::Sequencer
{

struct INonLinearTimeTransform
{
	virtual ~INonLinearTimeTransform() = default;

	virtual double SourceToView(double Seconds) const = 0;
	virtual double ViewToSource(double Source) const = 0;
};



/**
 * Utility for converting time units to slate pixel units and vice versa
 */
struct FTimeToPixelSpace
{
public:

	SEQUENCERCORE_API FTimeToPixelSpace( const FGeometry& AllottedGeometry, const TRange<double>& InLocalViewRange, const FFrameRate& InTickResolution );

	SEQUENCERCORE_API FTimeToPixelSpace( float WidthPx, const TRange<double>& InLocalViewRange, const FFrameRate& InTickResolution );

	SEQUENCERCORE_API FTimeToPixelSpace CreateNonLinear(TSharedPtr<INonLinearTimeTransform> InNonLinearTransform) const;

	/**
	 * Converts a time to a pixel point relative to the geometry of a widget (passed into the constructor)
	 *
	 * @param Time	The time to convert
	 * @return The pixel equivalent of the time
	 */
	SEQUENCERCORE_API float SecondsToPixel( double Time ) const;

	/**
	 * Converts a time delta to a pixel delta
	 *
	 * @param TimeDelta  The time delta to convert
	 * @return           The pixel equivalent of the delta time
	 */
	SEQUENCERCORE_API float SecondsDeltaToPixel( double TimeDelta ) const;

	/**
	 * Converts a pixel value to time 
	 *
	 * @param PixelX The x value of a pixel coordinate relative to the geometry that was passed into the constructor.
	 * @return The time where the pixel is located
	 */
	SEQUENCERCORE_API double PixelToSeconds( float PixelX ) const;

	/**
	 * Converts a frame time to a pixel point relative to the geometry of a widget (passed into the constructor)
	 *
	 * @param Time The time to convert
	 * @return The pixel equivalent of the frame time
	 */
	SEQUENCERCORE_API float FrameToPixel( const FFrameTime& Time ) const;

	/**
	 * Converts a frame delta value to pixel delta
	 *
	 * @param TimeDelta   The time delta to convert
	 * @return            The pixel equivalent of the delta time
	 */
	SEQUENCERCORE_API float FrameDeltaToPixel( const FFrameTime& TimeDelta) const;

	/**
	 * Converts a pixel value to frame time 
	 *
	 * @param PixelX The x value of a pixel coordinate relative to the geometry that was passed into the constructor.
	 * @return The frame time where the pixel is located
	 */
	SEQUENCERCORE_API FFrameTime PixelToFrame( float PixelX ) const;

	/**
	 * Converts a pixel delta value to delta frame time 
	 *
	 * @param PixelDelta The delta value in pixel space
	 * @return The equivalent delta frame time
	 */
	SEQUENCERCORE_API FFrameTime PixelDeltaToFrame( float PixelDelta ) const;

	/**
	 * Converts a pixel delta value to delta seconds time 
	 *
	 * @param PixelDelta The delta value in pixel space
	 * @return The equivalent delta time in seconds
	 */
	SEQUENCERCORE_API double PixelDeltaToSeconds( float PixelDelta ) const;

	/**
	 * Retrieve the tick resolution of the current sequence
	 */
	SEQUENCERCORE_API FFrameRate GetTickResolution() const;

	/**
	 * Make this converter relative to the specified time (ie, such that pixel 0 == FrameAmount)
	 */
	SEQUENCERCORE_API FTimeToPixelSpace RelativeTo(const FFrameTime& FrameZero) const;

	TSharedPtr<INonLinearTimeTransform> NonLinearTransform;

protected:
	FTimeToPixelSpace(){}

	/** The tick resolution of the current timeline */
	FFrameRate TickResolution;
	/** The number of pixels in the view range */
	float PixelsPerSecond;
	float PixelOffset;
};



} // namespace UE::Sequencer


/**
 * Utility for converting time units to slate pixel units and vice versa
 * This class will eventually be deprecated in favor of UE::Sequencer::FTimeToPixelSpace
 */
struct FTimeToPixel : UE::Sequencer::FTimeToPixelSpace
{
public:

	FTimeToPixel() = delete;

	FTimeToPixel( const FGeometry& AllottedGeometry, const TRange<double>& InLocalViewRange, const FFrameRate& InTickResolution )
		: UE::Sequencer::FTimeToPixelSpace(AllottedGeometry.GetLocalSize().X, InLocalViewRange, InTickResolution)
	{
	}

	FTimeToPixel( float WidthPx, const TRange<double>& InLocalViewRange, const FFrameRate& InTickResolution )
		: UE::Sequencer::FTimeToPixelSpace(WidthPx, InLocalViewRange, InTickResolution)
	{
	}

	FTimeToPixel(const UE::Sequencer::FTimeToPixelSpace& InOther)
		: UE::Sequencer::FTimeToPixelSpace(InOther)
	{
	}

	FTimeToPixel CreateNonLinear(TSharedPtr<UE::Sequencer::INonLinearTimeTransform> InNonLinearTransform) const
	{
		FTimeToPixel Copy = *this;
		Copy.NonLinearTransform = InNonLinearTransform;
		return Copy;
	}

	FTimeToPixel& operator=(const UE::Sequencer::FTimeToPixelSpace& InOther)
	{
		static_cast<UE::Sequencer::FTimeToPixelSpace&>(*this) = InOther;
		return *this;
	}

	/**
	 * Make this converter relative to the specified time (ie, such that pixel 0 == FrameAmount)
	 */
	SEQUENCERCORE_API FTimeToPixel RelativeTo(const FFrameTime& FrameZero) const;
};
