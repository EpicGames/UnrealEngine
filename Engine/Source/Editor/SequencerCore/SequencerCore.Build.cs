// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SequencerCore : ModuleRules
{
	public SequencerCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"CurveEditor",
				"Engine",
				"GraphEditor",
				"InputCore",
				"Slate",
				"SlateCore",
				"TimeManagement",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"EditorFramework",
				"EditorStyle",
				"UnrealEd",
			}
		);
	}
}
