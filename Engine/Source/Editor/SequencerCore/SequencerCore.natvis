<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="UE::Sequencer::FViewModelListHead">
	<AlternativeType Name="UE::Sequencer::FViewModelListHead*"/>
	<DisplayString>{ Type }</DisplayString>
    <Expand>
      <CustomListItems>

		<Variable Name="Count" InitialValue="0" />
		<Variable Name="CurrentItem" InitialValue="(UE::Sequencer::FViewModel*)HeadLink.Next.Object" />
		<Loop>
			<Break Condition="CurrentItem == 0" />
			<Exec>Count = Count + 1</Exec>
			<Exec>CurrentItem = CurrentItem->Link.Next.Object</Exec>
		</Loop>
		  
		<Item Name="Num">Count</Item>
		<Exec>Count = 0</Exec>
		<Exec>CurrentItem = (UE::Sequencer::FViewModel*)HeadLink.Next.Object</Exec>

        <Loop>
          <Break Condition="CurrentItem == 0" />
          <Item Name="[{Count}]">CurrentItem</Item>
          <Exec>Count = Count + 1</Exec>
          <Exec>CurrentItem = CurrentItem->Link.Next.Object</Exec>
        </Loop>
      </CustomListItems>
      <Item Name="NextListHead">NextListHead</Item>
    </Expand>
  </Type>

  <Type Name="UE::Sequencer::FViewModelListLink">
	  <DisplayString>Next = { (UE::Sequencer::FViewModel*)Next.Object }</DisplayString>
	  <Expand>
		  <Item Name="Next">(UE::Sequencer::FViewModel*)Next.Object</Item>
		  <Item Name="Previous" Condition="WeakPrev.WeakReferenceCount.ReferenceController->SharedReferenceCount._Storage._Value == 0">0</Item>
		  <Item Name="Previous" Condition="WeakPrev.WeakReferenceCount.ReferenceController->SharedReferenceCount._Storage._Value != 0 &amp;&amp; WeakPrev.Object->WeakPrev.WeakReferenceCount.ReferenceController->SharedReferenceCount._Storage._Value > 0">WeakPrev.Object->WeakPrev.Object->Next</Item>
	  </Expand>
  </Type>

  <Type Name="UE::Sequencer::FViewModel">

	<Expand>

		<Synthetic Name="Parent">
			<DisplayString>{ WeakParent.Object }</DisplayString>
			<Expand>
				<ExpandedItem>WeakParent.Object</ExpandedItem>
			</Expand>
		</Synthetic>
		<Item Name="NextSibling" ExcludeView="raw">(UE::Sequencer::FViewModel*)Link.Next.Object</Item>
		<Item Name="PreviousSibling" ExcludeView="raw" Condition="Link.WeakPrev.WeakReferenceCount.ReferenceController->SharedReferenceCount._Storage._Value != 0 &amp;&amp; Link.WeakPrev.Object->WeakPrev.WeakReferenceCount.ReferenceController->SharedReferenceCount._Storage._Value > 0">Link.WeakPrev.Object->WeakPrev.Object->Next</Item>
		<Synthetic Name="Children" ExcludeView="raw">
			<DisplayString>{ FirstChildListHead }</DisplayString>
			<Expand>
				<ExpandedItem>FirstChildListHead</ExpandedItem>
			</Expand>
		</Synthetic>
		<ExpandedItem>((UE::Sequencer::FViewModel*)this),!</ExpandedItem>
    </Expand>
  </Type>
</AutoVisualizer>
